// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationController : ModuleRules
	{
		public AutomationController(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"AutomationTest",
				}
			); 
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                    "AssetRegistry",
					"AutomationMessages",
                    "Json",
                    "JsonUtilities",
					"ScreenShotComparisonTools",
					"HTTP",
                    "AssetRegistry"
				}
			);

            if (Target.bBuildEditor)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[] {
						"EditorFramework",
						"MessageLog",
						"UnrealEdMessages",
						"UnrealEd",
                        "Engine", // Needed for UWorld/GWorld to find current level
				    }
                );
            }

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"MessagingCommon"
				}
			);

			if (Target.bCompileAgainstEngine && Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrecompileForTargets = PrecompileTargetsType.Any;
			}
		}
	}
}
