// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/Guid.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Containers/Ticker.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Misc/FilterCollection.h"
#include "IAutomationControllerModule.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "AutomationControllerSettings.h"
#include "AutomationGroupFilter.h"
#include "Containers/Ticker.h"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationCommandLine, Log, All);

/** States for running the automation process */
enum class EAutomationTestState : uint8
{
	Idle,				// Automation process is not running
	Initializing,		//
	FindWorkers,		// Find workers to run the tests
	RequestTests,		// Find the tests that can be run on the workers
	DoingRequestedWork,	// Do whatever was requested from the commandline
	Complete			// The process is finished
};

enum class EAutomationCommand : uint8
{
	ListAllTests,			//List all tests for the session
	RunCommandLineTests,	//Run only tests that are listed on the commandline
	RunAll,					//Run all the tests that are supported
	RunFilter,              //Run only tests that are tagged with this filter
	Quit,					//quit the app when tests are done, uses forced exit
	SoftQuit				//quit the app when tests are done without forced exit
};


class FAutomationExecCmd : private FSelfRegisteringExec
{
public:
	static const float DefaultDelayTimer;
	static const float DefaultFindWorkersTimeout;

	FAutomationExecCmd()
	{
		DelayTimer = DefaultDelayTimer;
		FindWorkersTimeout = DefaultFindWorkersTimeout;
		FindWorkerAttempts = 0;
		TestCount = 0;
	}

	void Init()
	{
		SessionID = FApp::GetSessionId();

		// Set state to FindWorkers to kick off the testing process
		AutomationTestState = EAutomationTestState::Initializing;
		DelayTimer = DefaultDelayTimer;

		// Load the automation controller
		IAutomationControllerModule* AutomationControllerModule = &FModuleManager::LoadModuleChecked<IAutomationControllerModule>("AutomationController");
		AutomationController = AutomationControllerModule->GetAutomationController();

		AutomationController->Init();
		FAutomationTestFramework::Get().LoadTestTagMappings();

		//TODO AUTOMATION Always use fullsize screenshots.
		const bool bFullSizeScreenshots = FParse::Param(FCommandLine::Get(), TEXT("FullSizeScreenshots"));
		const bool bSendAnalytics = FParse::Param(FCommandLine::Get(), TEXT("SendAutomationAnalytics"));

		// Register for the callback that tells us there are tests available
		if (!TestsRefreshedHandle.IsValid()) {
			TestsRefreshedHandle = AutomationController->OnTestsRefreshed().AddRaw(this, &FAutomationExecCmd::HandleRefreshTestCallback);
		}

		if (!TickHandler.IsValid()) {
			TickHandler = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FAutomationExecCmd::Tick));
		}

		int32 NumTestLoops = 1;
		FParse::Value(FCommandLine::Get(), TEXT("TestLoops="), NumTestLoops);
		AutomationController->SetNumPasses(NumTestLoops);
		SetUpFilterMapping();
	}

	void SetUpFilterMapping()
	{
		FilterMaps.Empty();
		FilterMaps.Add("Engine", EAutomationTestFlags::EngineFilter);
		FilterMaps.Add("Smoke", EAutomationTestFlags::SmokeFilter);
		FilterMaps.Add("Stress", EAutomationTestFlags::StressFilter);
		FilterMaps.Add("Perf", EAutomationTestFlags::PerfFilter);
		FilterMaps.Add("Product", EAutomationTestFlags::ProductFilter);
		FilterMaps.Add("Standard", EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::PerfFilter);
		FilterMaps.Add("Negative", EAutomationTestFlags::NegativeFilter);
		FilterMaps.Add("All", EAutomationTestFlags_FilterMask);
	}
	
	void Shutdown()
	{
		IAutomationControllerModule* AutomationControllerModule = FModuleManager::GetModulePtr<IAutomationControllerModule>("AutomationController");
		if ( AutomationControllerModule )
		{
			AutomationController = AutomationControllerModule->GetAutomationController();
			AutomationController->OnTestsRefreshed().RemoveAll(this);
		}

		FTSTicker::RemoveTicker(TickHandler);
	}

	bool IsTestingComplete()
	{
		// If the automation controller is no longer processing and we've reached the final stage of testing
		if ((AutomationController->GetTestState() != EAutomationControllerModuleState::Running) && (AutomationTestState == EAutomationTestState::Complete) && (AutomationCommandQueue.Num() == 0))
		{
			UE_LOGF(LogAutomationCommandLine, Display, "...Automation Test Queue Empty %d tests performed.", TestCount);
			TestCount = 0;
			return true;
		}
		return false;
	}

	void GenerateTestNamesFromCommandLine(TSharedPtr <AutomationFilterCollection> InFilters, TArray<FString>& OutFilteredTestNames)
	{
		OutFilteredTestNames.Empty();
		
		//Split the argument names up on +
		TArray<FString> ArgumentNames;
		StringCommand.ParseIntoArray(ArgumentNames, TEXT("+"), true);

		// get our settings CDO where things are stored
		UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();

		// iterate through the arguments to build a filter list by doing the following -
		// 1) If argument is a filter (StartsWith:system) then make sure we only filter-in tests that start with that filter
		// 2) If argument is a group then expand that group into multiple filters based on ini entries
		// 3) Otherwise just substring match (default behavior in 4.22 and earlier).
		FAutomationGroupFilter* FilterAny = new FAutomationGroupFilter();
		TArray<FAutomatedTestFilter> FiltersList;
		for (int32 ArgumentIndex = 0; ArgumentIndex < ArgumentNames.Num(); ++ArgumentIndex)
		{
			const FString GroupPrefix = TEXT("Group:");
			const FString FilterPrefix = TEXT("StartsWith:");

			FString ArgumentName = ArgumentNames[ArgumentIndex].TrimStartAndEnd();

			// if the argument is a filter (e.g. Filter:System) then create a filter that matches from the start
			if (ArgumentName.StartsWith(FilterPrefix))
			{
				FString FilterName = ArgumentName.RightChop(FilterPrefix.Len()).TrimStart();

				if (FilterName.EndsWith(TEXT(".")) == false)
				{
					FilterName += TEXT(".");
				}

				FAutomatedTestFilter StartsWithFilter(FilterName, true, false);

				if (!TagFilter.IsEmpty())
				{
					StartsWithFilter.Tags = GetTestTagFilter(TagFilter);
				}

				FiltersList.Add(StartsWithFilter);
			}
			else if (ArgumentName.StartsWith(GroupPrefix))
			{
				// if the argument is a group (e.g. Group:Rendering) then seach our groups for one that matches
				FString GroupName = ArgumentName.RightChop(GroupPrefix.Len()).TrimStart();

				bool FoundGroup = false;

				for (int32 i = 0; i < Settings->Groups.Num(); ++i)
				{
					FAutomatedTestGroup* GroupEntry = &(Settings->Groups[i]);
					if (GroupEntry && GroupEntry->Name == GroupName)
					{
						FoundGroup = true;
						// if found add all this groups filters to our current list
						if (GroupEntry->Filters.Num() > 0)
						{
							if (!TagFilter.IsEmpty())
							{
								auto TagsFilter = GetTestTagFilter(TagFilter);
								for (int32 j = 0; j < GroupEntry->Filters.Num(); ++j)
								{
									FAutomatedTestFilter* Filter = &(GroupEntry->Filters[j]);
									Filter->Tags = TagsFilter;
								}
							}
							FiltersList.Append(GroupEntry->Filters);
						}
						else
						{
							UE_LOGF(LogAutomationCommandLine, Warning, "Group %ls contains no filters", *GroupName);
						}
					}
				}

				if (!FoundGroup)
				{
					UE_LOGF(LogAutomationCommandLine, Error, "No matching group named %ls", *GroupName);
				}
			}
			else
			{
				bool bMatchFromStart = false;
				bool bMatchFromEnd = false;

				if (ArgumentName.StartsWith("^"))
				{
					bMatchFromStart = true;
					ArgumentName.RightChopInline(1);
				}
				if (ArgumentName.EndsWith("$"))
				{
					bMatchFromEnd = true;
					ArgumentName.LeftChopInline(1);
				}

				FiltersList.Add(FAutomatedTestFilter(ArgumentName, bMatchFromStart, bMatchFromEnd));
			}
		}

		if (!FiltersList.IsEmpty())
		{
			FilterAny->SetFilters(FiltersList);
			InFilters->Add(MakeShareable(FilterAny));

			// SetFilter applies all filters from the AutomationFilters array
			AutomationController->SetFilter(InFilters);
			// Fill OutFilteredTestNames array with filtered test names
			AutomationController->GetFilteredTestNames(OutFilteredTestNames);
		}
	}

	void FindWorkers(float DeltaTime)
	{
		DelayTimer -= DeltaTime;

		if (DelayTimer <= 0)
		{
			// Request the workers
			AutomationController->RequestAvailableWorkers(SessionID);
			AutomationTestState = EAutomationTestState::RequestTests;
			FindWorkersTimeout = DefaultFindWorkersTimeout;
			FindWorkerAttempts++;
		}
	}

	void RequestTests(float DeltaTime)
	{
		FindWorkersTimeout -= DeltaTime;

		if (FindWorkersTimeout <= 0)
		{
			// Call the refresh callback manually
			HandleRefreshTimeout();
		}
	}

	void HandleRefreshTimeout()
	{
		const float TimeOut = GetDefault<UAutomationControllerSettings>()->GameInstanceLostTimerSeconds;
		if (FindWorkerAttempts * DefaultFindWorkersTimeout >= TimeOut)
		{
			LogCommandLineError(FString::Printf(TEXT("Failed to find workers after %.02f seconds. Giving up"), TimeOut));
			AutomationTestState = EAutomationTestState::Complete;
		}
		else
		{
			// Go back to looking for workers
			UE_LOGF(LogAutomationCommandLine, Log, "Can't find any workers! Searching again");
			AutomationTestState = EAutomationTestState::FindWorkers;
		}
	}

	void ApplyCVarTagFilter(TSharedPtr <AutomationFilterCollection> AutomationFilters)
	{
		if (IConsoleVariable* TagCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Automation.TestTagGlobalFilter")))
		{
			FString TagFilterValue = TagCVar->GetString();
			if (!TagFilterValue.IsEmpty())
			{
				FAutomationTestFramework::Get().LoadTestTagMappings();

				UE_LOGF(LogAutomationCommandLine, Display, "Applying Automation Test tag filter '%ls'", *TagFilterValue);
				FAutomationGroupFilter* FilterTags = new FAutomationGroupFilter();
				FilterTags->SetTagFilter(GetTestTagFilter(TagFilterValue));
				AutomationFilters->Add(MakeShareable(FilterTags));
			}
		}
	}

	TArray<FAutomatedTestTagFilter> GetTestTagFilter(FString InTagFilter)
	{
		TArray<FAutomatedTestTagFilter> TagList;
		TagList.Add(FAutomatedTestTagFilter(InTagFilter));
		return TagList;
	}

	void HandleRefreshTestCallback()
	{
		TArray<FString> FilteredTestNames;

		// This is called by the controller manager when it receives responses. We want to make sure it has a device, and we
		// want to make sure it's called while we're waiting for a response
		if (AutomationController->GetNumDeviceClusters() == 0 || AutomationTestState != EAutomationTestState::RequestTests)
		{
			UE_LOGF(LogAutomationCommandLine, Log, "Ignoring refresh from ControllerManager. NumDeviceClusters=%d, CurrentState=%d", AutomationController->GetNumDeviceClusters(), int(AutomationTestState));
			return;
		}

		// We have found some workers
		// Create a filter to add to the automation controller, otherwise we don't get any reports
		TSharedPtr <AutomationFilterCollection> AutomationFilters = MakeShareable(new AutomationFilterCollection());
		ApplyCVarTagFilter(AutomationFilters);
		AutomationController->SetFilter(AutomationFilters);
		AutomationController->SetVisibleTestsEnabled(true);
		AutomationController->GetEnabledTestNames(FilteredTestNames);

		//assume we won't run any tests
		bool bRunTests = false;

		if (AutomationCommand == EAutomationCommand::ListAllTests)
		{
			UE_LOGF(LogAutomationCommandLine, Display, "Found %d Automation Tests", FilteredTestNames.Num());
			for ( const FString& TestName : FilteredTestNames)
			{
				UE_LOGF(LogAutomationCommandLine, Display, "\t'%ls'", *TestName);
			}

			// Set state to complete
			AutomationTestState = EAutomationTestState::Complete;
		}
		else if (AutomationCommand == EAutomationCommand::RunCommandLineTests)
		{
			GenerateTestNamesFromCommandLine(AutomationFilters, FilteredTestNames);
			
			if (FilteredTestNames.Num() == 0)
			{
				if (bAllowZeroTestResults)
				{
					UE_LOGF(LogAutomationCommandLine, Warning, "No automation tests matched '%ls'", *StringCommand);
				}
				else
				{
					LogCommandLineError(FString::Printf(TEXT("No automation tests matched '%s'"), *StringCommand));
				}
			}
			else
			{
				UE_LOGF(LogAutomationCommandLine, Display, "Found %d automation tests based on '%ls'", FilteredTestNames.Num(), *StringCommand);
			}
							
			for ( const FString& TestName : FilteredTestNames )
			{
				UE_LOGF(LogAutomationCommandLine, Display, "\t%ls", *TestName);
			}

			if (FilteredTestNames.Num())
			{
				bRunTests = true;
			}
			else
			{
				AutomationTestState = EAutomationTestState::Complete;
			}
		}
		else if (AutomationCommand == EAutomationCommand::RunFilter)
		{
			if (FilterMaps.Contains(StringCommand))
			{
				UE_LOGF(LogAutomationCommandLine, Display, "Running %i Automation Tests", FilteredTestNames.Num());
				if (FilteredTestNames.Num() > 0)
				{
					bRunTests = true;
				}
				else
				{
					AutomationTestState = EAutomationTestState::Complete;
				}
			}
			else
			{
				AutomationTestState = EAutomationTestState::Complete;
				UE_LOGF(LogAutomationCommandLine, Display, "%ls is not a valid flag to filter on! Valid options are: ", *StringCommand);
				TArray<FString> FlagNames;
				FilterMaps.GetKeys(FlagNames);
				for (int i = 0; i < FlagNames.Num(); i++)
				{
					UE_LOGF(LogAutomationCommandLine, Display, "\t%ls", *FlagNames[i]);
				}
			}
		}
		else if (AutomationCommand == EAutomationCommand::RunAll)
		{
			bRunTests = true;
		}

		if (bRunTests)
		{
			AutomationController->StopTests();
			AutomationController->SetEnabledTests(FilteredTestNames);
			TestCount = FilteredTestNames.Num();

			// Clear delegate to avoid re-running tests due to multiple delegates being added or when refreshing session frontend
			// The delegate will be readded in Init whenever a new command is executed
			AutomationController->OnTestsRefreshed().Remove(TestsRefreshedHandle);
			TestsRefreshedHandle.Reset();

			AutomationController->RunTests();

			// Set state to monitoring to check for test completion
			AutomationTestState = EAutomationTestState::DoingRequestedWork;
		}
	}

	void MonitorTests()
	{
		if (AutomationController->GetTestState() != EAutomationControllerModuleState::Running)
		{
			// We have finished the testing, and results are available
			AutomationTestState = EAutomationTestState::Complete;
		}
	}

	bool Tick(float DeltaTime)
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_FAutomationExecCmd_Tick);

		// Update the automation controller to keep it running
		AutomationController->Tick();

		// Update the automation process
		switch (AutomationTestState)
		{
			case EAutomationTestState::Initializing:
			{
				if (AutomationController->IsReadyForTests())
				{
					AutomationTestState = EAutomationTestState::Idle;
					UE_LOGF(LogAutomationCommandLine, Display, "Ready to start automation");
				}
				FindWorkerAttempts = 0;
				break;
			}
			case EAutomationTestState::FindWorkers:
			{
				FindWorkers(DeltaTime);
				break;
			}
			case EAutomationTestState::RequestTests:
			{
				RequestTests(DeltaTime);
				break;
			}
			case EAutomationTestState::DoingRequestedWork:
			{
				MonitorTests();
				break;
			}
			case EAutomationTestState::Complete:
			case EAutomationTestState::Idle:
			default:
			{
				//pop next command
				if (AutomationCommandQueue.Num())
				{
					AutomationCommand = AutomationCommandQueue[0];
					AutomationCommandQueue.RemoveAt(0);
					if (AutomationCommand == EAutomationCommand::Quit || AutomationCommand == EAutomationCommand::SoftQuit)
					{
						if (AutomationCommandQueue.IsValidIndex(0) && !IsQuitQueued())
						{
							// Add Quit and SoftQuit commands back to the end of the array.
							AutomationCommandQueue.Add(AutomationCommand);
							break;
						}
					}
					AutomationTestState = EAutomationTestState::FindWorkers;
				}

				// Only quit if Quit is the actual last element in the array.
				if (AutomationCommand == EAutomationCommand::Quit || AutomationCommand == EAutomationCommand::SoftQuit)
				{
					if (!GIsCriticalError)
					{
						if (AutomationController->ReportsHaveErrors() || Errors.Num())
						{
							UE_LOGF(LogAutomationCommandLine, Display, "Setting GIsCriticalError due to test failures (will cause non-zero exit code).");
							GIsCriticalError = true;
						}
					}
					UE_LOGF(LogAutomationCommandLine, Log, "Shutting down. GIsCriticalError=%d", GIsCriticalError ? 1 : 0);
					// some tools parse this.
					UE_LOGF(LogAutomationCommandLine, Display, "**** TEST COMPLETE. EXIT CODE: %d ****", GIsCriticalError ? -1 : 0);
					FPlatformMisc::RequestExitWithStatus(AutomationCommand == EAutomationCommand::SoftQuit ? false : true, GIsCriticalError ? -1 : 0);
					// We have finished the testing, and results are available
					AutomationTestState = EAutomationTestState::Complete;
				}
				else if (!IsAboutToRunTest())
				{
					// Register for the callback that tells us there are tests available
					if (!TestsRefreshedHandle.IsValid()) {
						TestsRefreshedHandle = AutomationController->OnTestsRefreshed().AddRaw(this, &FAutomationExecCmd::HandleRefreshTestCallback);
					}
				}
				break;
			}
		}

		if (IsTestingComplete())
		{
			AutomationTestState = EAutomationTestState::Idle;
			TickHandler.Reset();
			return false;
		}
		return true;
	}

	bool IsRunTestQueued()
	{
		for (auto Command : AutomationCommandQueue)
		{
			if (Command == EAutomationCommand::RunCommandLineTests
				|| Command == EAutomationCommand::RunAll
				|| Command == EAutomationCommand::RunFilter)
			{
				return true;
			}
		}

		return false;
	}

	bool IsAboutToRunTest()
	{
		return (AutomationCommand == EAutomationCommand::RunCommandLineTests
			|| AutomationCommand == EAutomationCommand::RunAll
			|| AutomationCommand == EAutomationCommand::RunFilter);
	}

	bool IsQuitQueued()
	{
		for (auto Command : AutomationCommandQueue)
		{
			if (Command == EAutomationCommand::Quit
				|| Command == EAutomationCommand::SoftQuit)
			{
				return true;
			}
		}

		return false;
	}
	
protected:
	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Dev(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		bool bHandled = false;
		// figure out if we are handling this request
		if (FParse::Command(&Cmd, TEXT("Automation")))
		{
			// Early exit in case of a CVar input. ie: Automation.SkipStackWalk 1
			if (FString(Cmd).StartsWith(TEXT(".")))
			{
				return false;
			}

			// Track whether we have a flag we care about passing through.
			FString FlagToUse = "";

			TArray<FString> CommandList;
			FString(Cmd).ParseIntoArray(CommandList, TEXT(";"), true);

			Init();

			//assume we handle this
			bHandled = true;

			for (int CommandIndex = 0; CommandIndex < CommandList.Num(); ++CommandIndex)
			{
				const TCHAR* TempCmd = *CommandList[CommandIndex];
				if (FParse::Command(&TempCmd, TEXT("StartRemoteSession")))
				{
					FString SessionString = TempCmd;
					if (!FGuid::Parse(SessionString, SessionID))
					{
						Ar.Logf(TEXT("Automation: %s is not a valid session guid!"), *SessionString);
						bHandled = false;
						break;
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("List")))
				{
					AutomationCommandQueue.Add(EAutomationCommand::ListAllTests);
				}
				else if (FParse::Command(&TempCmd, TEXT("Now")))
				{
					DelayTimer = 0.0f;
				}
				else if (FParse::Command(&TempCmd, TEXT("RunTests")) || FParse::Command(&TempCmd, TEXT("RunTest")))
				{
					if (FParse::Command(&TempCmd, TEXT("Now")))
					{
						DelayTimer = 0.0f;
						continue;
					}

					//only one of these should be used
					if (IsRunTestQueued())
					{
						Ar.Logf(TEXT("Automation: A test run is already Queued: %s. Only one run is supported at a time."), *StringCommand);
						continue;
					}

					StringCommand = TempCmd;
					Ar.Logf(TEXT("Automation: RunTests='%s' Queued."), *StringCommand);
					AutomationCommandQueue.Add(EAutomationCommand::RunCommandLineTests);
				}
				else if (FParse::Command(&TempCmd, TEXT("SetMinimumPriority")))
				{
					FlagToUse = TempCmd;
					Ar.Logf(TEXT("Automation: Setting minimum priority of cases to run to: %s"), *FlagToUse);
					if (FlagToUse.Contains(TEXT("Low")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags_PriorityMask);
					}
					else if (FlagToUse.Contains(TEXT("Medium")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags_MediumPriorityAndAbove);
					}
					else if (FlagToUse.Contains(TEXT("High")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags_HighPriorityAndAbove);
					}
					else if (FlagToUse.Contains(TEXT("Critical")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::CriticalPriority);
					}
					else if (FlagToUse.Contains(TEXT("None")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::None);
					}
					else
					{
						Ar.Logf(TEXT("Automation: %s is not a valid priority!\nValid priorities are Critical, High, Medium, Low, None"), *FlagToUse);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("SetPriority")))
				{
					FlagToUse = TempCmd;
					Ar.Logf(TEXT("Setting explicit priority of cases to run to: %s"), *FlagToUse);
					if (FlagToUse.Contains(TEXT("Low")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::LowPriority);
					}
					else if (FlagToUse.Contains(TEXT("Medium")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::MediumPriority);
					}
					else if (FlagToUse.Contains(TEXT("High")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::HighPriority);
					}
					else if (FlagToUse.Contains(TEXT("Critical")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::CriticalPriority);
					}
					else if (FlagToUse.Contains(TEXT("None")))
					{
						AutomationController->SetRequestedTestFlags(EAutomationTestFlags::None);
					}

					else
					{
						Ar.Logf(TEXT("Automation: %s is not a valid priority!\nValid priorities are Critical, High, Medium, Low, None"), *StringCommand);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("RunFilter")))
				{
					//only one of these should be used
					if (IsRunTestQueued())
					{
						Ar.Logf(TEXT("Automation: A test run is already Queued: %s. Only one run is supported at a time."), *StringCommand);
						continue;
					}
					FlagToUse = TempCmd;
					StringCommand = TempCmd;
					if (FilterMaps.Contains(FlagToUse))
					{
						AutomationController->SetRequestedTestFlags(FilterMaps[FlagToUse]);
						Ar.Logf(TEXT("Automation: RunFilter='%s' Queued."), *FlagToUse);
					}
					AutomationCommandQueue.Add(EAutomationCommand::RunFilter);
				}
				else if (FParse::Command(&TempCmd, TEXT("SetFilter")))
				{
					FlagToUse = TempCmd;
					if (FilterMaps.Contains(FlagToUse))
					{
						AutomationController->SetRequestedTestFlags(FilterMaps[FlagToUse]);
						Ar.Logf(TEXT("Automation: Setting test filter: %s"), *FlagToUse);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("RunAll")))
				{
					//only one of these should be used
					if (IsRunTestQueued())
					{
						Ar.Logf(TEXT("Automation: A test run is already Queued: %s. Only one run is supported at a time."), *StringCommand);
						continue;
					}
					AutomationCommandQueue.Add(EAutomationCommand::RunAll);
					AutomationController->SetRequestedTestFlags(FilterMaps["All"]);
					Ar.Logf(TEXT("Automation: RunAll Queued. NOTE: This may take a while."));
				}
				else if (FParse::Command(&TempCmd, TEXT("Quit")))
				{
					if (IsQuitQueued())
					{
						Ar.Log(TEXT("Automation: Quit command is already Queued."));
						continue;
					}
					AutomationCommandQueue.Add(EAutomationCommand::Quit);
					Ar.Logf(TEXT("Automation: Quit Command Queued."));
				}
				else if (FParse::Command(&TempCmd, TEXT("SoftQuit")))
				{
					if (IsQuitQueued())
					{
						Ar.Log(TEXT("Automation: Quit command is already Queued."));
						continue;
					}
					AutomationCommandQueue.Add(EAutomationCommand::SoftQuit);
					Ar.Logf(TEXT("Automation: SoftQuit Command Queued."));
				}
				else if (FParse::Command(&TempCmd, TEXT("IgnoreLogEvents")))
				{
					if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Automation.CaptureLogEvents")))
					{
						Ar.Logf(TEXT("Automation: Suppressing Log Events"));
						CVar->Set(false);
					}					
				}
				else if (FParse::Command(&TempCmd, TEXT("EnableStereoTests")))
				{
					if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Automation.EnableStereoTestVariants")))
					{
						Ar.Logf(TEXT("Automation: Enabling Stereo Test Variants"));
						CVar->Set(true);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("SetTagFilter")))
				{
					if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Automation.TestTagGlobalFilter")))
					{
						FString FilterValue(TempCmd);
						Ar.Logf(TEXT("Automation: Setting test tag global filter to: '%s'"), *FilterValue);
						CVar->Set(*FilterValue);
					}
				}
				else if (FParse::Command(&TempCmd, TEXT("ApplyTagFilter")))
				{
					TagFilter = FString(TempCmd).TrimStartAndEnd();
					Ar.Logf(TEXT("Automation: Storing tag filter to be applied: '%s'"), *TagFilter);
				}
				else if (FParse::Command(&TempCmd, TEXT("AllowZeroTestResults")))
				{
					bAllowZeroTestResults = true;
				}
				else if (FParse::Command(&TempCmd, TEXT("Help")))
				{
					Ar.Logf(TEXT("Supported commands are: "));
					Ar.Logf(TEXT("\tAutomation StartRemoteSession <sessionid>"));
					Ar.Logf(TEXT("\tAutomation List"));
					Ar.Logf(TEXT("\tAutomation RunTests <test string>"));
					Ar.Logf(TEXT("\tAutomation RunAll"));
					Ar.Logf(TEXT("\tAutomation RunFilter <filter name>"));
					Ar.Logf(TEXT("\tAutomation SetFilter <filter name>"));
					Ar.Logf(TEXT("\tAutomation SetMinimumPriority <minimum priority>"));
					Ar.Logf(TEXT("\tAutomation SetPriority <priority>"));
					Ar.Logf(TEXT("\tAutomation Now"));
					Ar.Logf(TEXT("\tAutomation Quit"));
					Ar.Logf(TEXT("\tAutomation SoftQuit"));
					Ar.Logf(TEXT("\tAutomation IgnoreLogEvents"));
					Ar.Logf(TEXT("\tAutomation EnableStereoTests"));
					Ar.Logf(TEXT("\tAutomation SetTagFilter"));
					Ar.Logf(TEXT("\tAutomation ApplyTagFilter"));
					Ar.Logf(TEXT("\tAutomation AllowZeroTestResults"));
					bHandled = false;
				}
				else
				{
					Ar.Logf(TEXT("Unknown Automation command '%s'! Use Help command for a detailed list."), TempCmd);
					bHandled = false;
				}
			}
		}
		
		// Shutdown our service
		return bHandled;
	}

private:

	// Logs and tracks an error
	void LogCommandLineError(const FString& InErrorMsg)
	{
		UE_LOGF(LogAutomationCommandLine, Error, "%ls", *InErrorMsg);
		Errors.Add(InErrorMsg);
	}

	/** The automation controller running the tests */
	IAutomationControllerManagerPtr AutomationController;

	/** The current state of the automation process */
	EAutomationTestState AutomationTestState;

	/** What work was requested */
	TArray<EAutomationCommand> AutomationCommandQueue;

	/** What work was requested */
	EAutomationCommand AutomationCommand;

	/** Handle to Test Refresh delegate */
	FDelegateHandle TestsRefreshedHandle;

	/** Delay used before finding workers on game instances. Just to ensure they have started up */
	float DelayTimer;

	/** Timer Handle for giving up on workers */
	float FindWorkersTimeout;

	/** How many times we attempted to find a worker... */
	int	 FindWorkerAttempts;

	/** Holds the session ID */
	FGuid SessionID;

	//so we can release control of the app and just get ticked like all other systems
	FTSTicker::FDelegateHandle TickHandler;

	//Extra commandline params
	FString StringCommand;

	//This is the numbers of tests that are found in the command line.
	int32 TestCount;

	//Dictionary that maps flag names to flag values.
	TMap<FString, EAutomationTestFlags> FilterMaps;

	// Any that we encountered during processing. Used in 'Quit' to determine error code
	TArray<FString> Errors;

	// Tag filter expression to be set to each group
	FString TagFilter;

	// If true, do not return non-zero exit code when no tests were run
	bool bAllowZeroTestResults = false;
};

const float FAutomationExecCmd::DefaultDelayTimer = 5.0f;
const float FAutomationExecCmd::DefaultFindWorkersTimeout = 30.0f;
static FAutomationExecCmd AutomationExecCmd;

void EmptyLinkFunctionForStaticInitializationAutomationExecCmd()
{
	// This function exists to prevent the object file containing this test from
	// being excluded by the linker, because it has no publicly referenced symbols.
}

