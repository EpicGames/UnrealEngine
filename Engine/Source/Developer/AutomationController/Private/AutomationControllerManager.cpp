// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationControllerManager.h"
#include "IAutomationControllerModule.h"
#include "Misc/AutomationEvent.h"
#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "IAutomationReport.h"
#include "AutomationWorkerMessages.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Modules/ModuleManager.h"
#include "ImageComparer.h"
#include "AutomationControllerSettings.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "Serialization/JsonSerializer.h"
#include "JsonObjectConverter.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "PlatformHttp.h"
#include "AutomationTelemetry.h"
#include "AssetRegistry/AssetRegistryModule.h"

#if WITH_EDITOR
#include "UnrealEdGlobals.h"
#include "AssetEditorMessages.h"
#include "Editor/UnrealEdEngine.h"
#include "Logging/MessageLog.h"
#include "Tests/AutomationCommon.h"
#endif
#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationControllerManager)

// these strings are parsed by Gauntlet (AutomationLogParser) so make sure changes are replicated there!
#define AutomationTestStarting		TEXT("Test Started. Name={%s} Path={%s}")
#define AutomationStateFormat		TEXT("Test Completed. Result={%s} Name={%s} Path={%s}")

#define BeginEventsFormat			TEXT("BeginEvents: %s")
#define EndEventsFormat				TEXT("EndEvents: %s")

DEFINE_LOG_CATEGORY_STATIC(LogAutomationController, Log, All)

void FAutomationResultEmitter::EmitTestStarted(const FString& Name, const FString& Path)
{
	UE_LOG(LogAutomationController, Display, AutomationTestStarting, *Name, *Path);
}

void FAutomationResultEmitter::EmitTestCompleted(const FString& State, const FString& Name, const FString& Path)
{
	UE_LOG(LogAutomationController, Display, AutomationStateFormat, *State, *Name, *Path);
}

void FAutomationResultEmitter::EmitBeginEvents(const FString& Path)
{
	UE_LOG(LogAutomationController, Log, BeginEventsFormat, *Path);
}

void FAutomationResultEmitter::EmitEndEvents(const FString& Path)
{
	UE_LOG(LogAutomationController, Log, EndEventsFormat, *Path);
}

void FAutomationResultEmitter::EmitEntry(const FAutomationExecutionEntry& Entry)
{
	switch (Entry.Event.Type)
	{
	case EAutomationEventType::Error:
		UE_LOG(LogAutomationController, Error, TEXT("%s"), *Entry.ToString());
		break;
	case EAutomationEventType::Warning:
		UE_LOG(LogAutomationController, Warning, TEXT("%s"), *Entry.ToString());
		break;
	default:
		UE_LOG(LogAutomationController, Log, TEXT("%s"), *Entry.ToString());
		break;
	}
}

#define LOCTEXT_NAMESPACE "AutomationTesting"

void FAutomatedTestPassResults::AddTestResult(const IAutomationReportPtr& TestReport)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	const FString& FullTestPath = TestReport->GetFullTestPath();
	if (!TestsMapIndex.Contains(FullTestPath))
	{
		FAutomatedTestResult TestResult;
		TestResult.Test = TestReport;
		TestResult.TestDisplayName = TestReport->GetDisplayName();
		TestResult.FullTestPath = FullTestPath;
		TestResult.Tags = TestReport->GetTagsArray();

		TestsMapIndex.Add(FullTestPath, Tests.Num());
		Tests.Add(TestResult);
		NotRun++;
	}
}

FAutomatedTestResult& FAutomatedTestPassResults::GetTestResult(const IAutomationReportPtr& TestReport)
{
	const FString& FullTestPath = TestReport->GetFullTestPath();
	check(TestsMapIndex.Contains(FullTestPath));
	return Tests[TestsMapIndex[FullTestPath]];
}

void FAutomatedTestPassResults::ReBuildTestsMapIndex()
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	TestsMapIndex.Empty();
	uint32 Index = 0;
	for (const FAutomatedTestResult& TestResult : Tests)
	{
		TestsMapIndex.Add(TestResult.FullTestPath, Index++);
	}
}

bool FAutomatedTestPassResults::ReflectResultStateToReport(IAutomationReportPtr& TestReport)
{
	const FString& FullTestPath = TestReport->GetFullTestPath();
	if (TestsMapIndex.Contains(FullTestPath))
	{
		FAutomatedTestResult& TestResult = Tests[TestsMapIndex[FullTestPath]];
		if (TestResult.State == EAutomationState::InProcess)
		{
			// Marking incomplete test from previous pass as failure.
			TestResult.AddEvent(EAutomationEventType::Error, TEXT("Test did not run until completion. The test exited prematurely."));
			TestResult.State = EAutomationState::Fail;
			InProcess--;
			Failed++;
		}
		TestReport->SetState(TestResult.State);
		TestResult.Test = TestReport;
		return true;
	}
	return false;
}

void FAutomatedTestPassResults::UpdateTestResultStatus(const IAutomationReportPtr& TestReport, EAutomationState State, bool bHasWarning)
{
	FAutomatedTestResult& TestResult = GetTestResult(TestReport);
	if (TestResult.State == State)
		return;

	// Book keeping
	// from transient states
	switch (TestResult.State)
	{
	case EAutomationState::InProcess:
		InProcess--;
		break;
	case EAutomationState::NotRun:
		NotRun--;
		break;
	}
	// to new state
	switch (State)
	{
	case EAutomationState::Success:
		if (bHasWarning)
		{
			SucceededWithWarnings++;
		}
		else
		{
			Succeeded++;
		}
		break;
	case EAutomationState::Fail:
		Failed++;
		break;
	case EAutomationState::InProcess:
		TestResult.DateTime = FDateTime::UtcNow();
		InProcess++;
		break;
	case EAutomationState::NotRun:
		NotRun++;
		break;
	case EAutomationState::Skipped:
		// Those are not counted
		break;
	}
	// commit the change
	TestResult.State = State;
}

FAutomationControllerManager::FAutomationControllerManager()
{

	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();

	if (Settings->CheckTestIntervalSeconds > 0.0f)
	{
		CheckTestIntervalSeconds = Settings->CheckTestIntervalSeconds;
	}
	
	if (Settings->GameInstanceLostTimerSeconds > 0.0f)
	{
		GameInstanceLostTimerSeconds = Settings->GameInstanceLostTimerSeconds;
	}

	bKeepPIEOpen = Settings->bKeepPIEOpen;
	bSortTestsByFailure = Settings->bSortTestsByFailure;
	bPruneLogsOnSuccess = Settings->bPruneLogsOnSuccess;
	
	FString DeveloperPath;
	FParse::Value(FCommandLine::Get(), TEXT("ReportOutputPath="), ReportExportPath, false);
	FParse::Value(FCommandLine::Get(), TEXT("DisplayReportOutputPath="), ReportURLPath, false);
	FParse::Value(FCommandLine::Get(), TEXT("DeveloperReportOutputPath="), DeveloperPath, false);

	if (ReportExportPath.Len())
	{
		UE_LOGF(LogAutomationController, Warning, "Argument -ReportOutputPath= is now -ReportExportPath=. Please update your command line!");
	}

	if (ReportURLPath.Len())
	{
		UE_LOGF(LogAutomationController, Warning, "Argument -DisplayReportOutputPath= is now -ReportURL=. Please update your command line!");
	}

	if (DeveloperPath.Len())
	{
		UE_LOGF(LogAutomationController, Warning, "Argument -DeveloperReportOutputPath= is now -DeveloperReport (no value). Please update your command line!");
	}

	// read values with new names
	FParse::Value(FCommandLine::Get(), TEXT("ReportExportPath="), ReportExportPath, false);
	FParse::Value(FCommandLine::Get(), TEXT("ReportURL="), ReportURLPath, false);
	bool bUseDeveloperPath = DeveloperPath.Len() > 0 || FParse::Param(FCommandLine::Get(), TEXT("DeveloperReport"));

	if (ReportExportPath.Len() && bUseDeveloperPath)
	{
		ReportExportPath = ReportExportPath / TEXT("dev") / FString(FPlatformProcess::UserName()).ToLower();
	}
		
	if (ReportURLPath.Len() && bUseDeveloperPath)
	{
		DeveloperReportUrl = ReportURLPath / TEXT("dev") / FString(FPlatformProcess::UserName()).ToLower() / TEXT("index.html");
	}

	bResumeRunTest = FParse::Param(FCommandLine::Get(), TEXT("ResumeRunTest"));
}

bool FAutomationControllerManager::IsReadyForTests()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		return false;
	}

#if WITH_EDITOR && !UE_BUILD_SHIPPING && WITH_AUTOMATION_TESTS
	if (InteractiveFrameRateCheck == nullptr)
	{
		InteractiveFrameRateCheck = MakeShared<FWaitForInteractiveFrameRate>();
	}

	if (!InteractiveFrameRateCheck->Update())
	{
		return false;
	}

	InteractiveFrameRateCheck = nullptr;
#endif // WITH_EDITOR
	return true;
}

void FAutomationControllerManager::RequestAvailableWorkers(const FGuid& SessionId)
{
	//invalidate previous tests
	++ExecutionCount;
	DeviceClusterManager.Reset();

	ControllerResetDelegate.Broadcast();

	// Don't allow reports to be exported
	bTestResultsAvailable = false;

	//store off active session ID to reject messages that come in from different sessions
	ActiveSessionId = SessionId;

	//EAutomationTestFlags_FilterMask

	//TODO AUTOMATION - include change list, game, etc, or remove when launcher is integrated
	int32 ChangelistNumber = 10000;
	FString ProcessName = TEXT("instance_name");

	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FAutomationWorkerFindWorkers>(ChangelistNumber, FApp::GetProjectName(), ProcessName, SessionId), EMessageScope::Network);

	// Reset the check test timers
	LastTimeUpdateTicked = FPlatformTime::Seconds();
	CheckTestTimer = 0.f;

	IScreenShotToolsModule& ScreenShotModule = FModuleManager::LoadModuleChecked<IScreenShotToolsModule>("ScreenShotComparisonTools");
	ScreenshotManager = ScreenShotModule.GetScreenShotManager();
}

void FAutomationControllerManager::RequestTests()
{
	//invalidate incoming results
	ExecutionCount++;
	//reset the number of responses we have received
	RefreshTestResponses = 0;

	ReportManager.Empty();

	for ( int32 ClusterIndex = 0; ClusterIndex < DeviceClusterManager.GetNumClusters(); ++ClusterIndex )
	{
		int32 DevicesInCluster = DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex);
		if ( DevicesInCluster > 0 )
		{
			FMessageAddress MessageAddress = DeviceClusterManager.GetDeviceMessageAddress(ClusterIndex, 0);

			UE_LOGF(LogAutomationController, Log, "Requesting test list from %ls", *MessageAddress.ToString());

			//issue tests on appropriate platforms
			SendMessage(
				FMessageEndpoint::MakeMessage<FAutomationWorkerRequestTests>(bDeveloperDirectoryIncluded, RequestedTestFlags),
				FAutomationWorkerRequestTests::StaticStruct(),
				MessageAddress);
		}
	}
}

void FAutomationControllerManager::RunTests(const bool bInIsLocalSession)
{
	ExecutionCount++;
	CurrentTestPass = 0;
	ReportManager.SetCurrentTestPass(CurrentTestPass);
	ClusterDistributionMask = 0;
	bTestResultsAvailable = false;
	TestRunningArray.Empty();
	bIsLocalSession = bInIsLocalSession;

	// Reset the check test timers
	LastTimeUpdateTicked = FPlatformTime::Seconds();
	CheckTestTimer = 0.f;

#if WITH_EDITOR
	FMessageLog AutomationEditorLog("AutomationTestingLog");
	FString NewPageName = FString::Printf(TEXT("-----Test Run %d----"), ExecutionCount);
	FText NewPageNameText = FText::FromString(*NewPageName);
	AutomationEditorLog.Open();
	AutomationEditorLog.NewPage(NewPageNameText);
	AutomationEditorLog.Info(NewPageNameText);
#endif
	//reset all tests
	ReportManager.ResetForExecution(NumTestPasses);

	// Register all tests that we'll need to be exported as json report
	bRequiresReportGeneration = (TestReportGeneratedDelegate.IsBound() || !ReportExportPath.IsEmpty());
	if (bRequiresReportGeneration)
	{
		JsonTestPassResults.IsRequired = true;
		TArray<IAutomationReportPtr> TestsToRun = ReportManager.GetEnabledTestReports();

		// Load previous test results
		if (bResumeRunTest)
		{
			if (!ReportExportPath.IsEmpty())
			{
				FString ReportFileName = FString::Printf(TEXT("%s/index.json"), *ReportExportPath);
				if (FPaths::FileExists(ReportFileName))
				{
					LoadJsonTestPassSummary(ReportFileName, TestsToRun);
				}
				else
				{
					UE_LOGF(LogAutomationController, Warning, "Cannot load previous test results. ReportFileName '%ls' was not found.", *ReportFileName);
				}
			}
			else
			{
				UE_LOGF(LogAutomationController, Warning, "Cannot load previous test results. Report export path not provided.");
			}
		}

		for (IAutomationReportPtr& TestReport : TestsToRun)
		{
			JsonTestPassResults.AddTestResult(TestReport);
		}

		if (!ReportExportPath.IsEmpty())
		{
			// Get the html index written down as soon as possible so once results are updated, they can be reviewed.
			GenerateTestPassHtmlIndex();
		}
	}

	for ( int32 ClusterIndex = 0; ClusterIndex < DeviceClusterManager.GetNumClusters(); ++ClusterIndex )
	{
		//enable each device cluster
		ClusterDistributionMask |= ( 1 << ClusterIndex );

		//for each device in this cluster
		for ( int32 DeviceIndex = 0; DeviceIndex < DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex); ++DeviceIndex )
		{
			//mark the device as idle
			DeviceClusterManager.SetTest(ClusterIndex, DeviceIndex, NULL);

			// Send command to start test session and reset tests (delete local files, etc)
			FMessageAddress MessageAddress = DeviceClusterManager.GetDeviceMessageAddress(ClusterIndex, DeviceIndex);
			UE_LOGF(LogAutomationController, Log, "Sending StartTestSession to %ls", *MessageAddress.ToString());

			SendMessage(
				FMessageEndpoint::MakeMessage<FAutomationWorkerStartTestSession>(),
				FAutomationWorkerStartTestSession::StaticStruct(),
				MessageAddress);

			StartedTestSessionWorkerInstanceIdSet.Add(DeviceClusterManager.GetClusterGameInstanceId(ClusterIndex, DeviceIndex));

			// Store devices info into the json report.
			if (JsonTestPassResults.IsRequired)
			{
				const FAutomationDeviceInfo& DeviceInfo = DeviceClusterManager.GetDeviceInfo(ClusterIndex, DeviceIndex);
				JsonTestPassResults.Devices.Add(DeviceInfo);
			}
		}
	}
	
	// Clear the logs and transient folders. Reports can be manually cleared by the user in the UI if they so desire
	UE_LOGF(LogAutomationController, Display, "Clearing %ls and %ls", *FPaths::AutomationLogDir(), *FPaths::AutomationTransientDir());
	IFileManager::Get().DeleteDirectory(*FPaths::AutomationLogDir(), false, true);
	IFileManager::Get().DeleteDirectory(*FPaths::AutomationTransientDir(), false, true);


	// Inform the UI we are running tests
	if ( ClusterDistributionMask != 0 )
	{
		SetControllerStatus(EAutomationControllerModuleState::Running);
	}
}

void FAutomationControllerManager::StopTests()
{
	bTestResultsAvailable = false;
	ClusterDistributionMask = 0;

	ReportManager.StopRunningTests();

	// Inform the UI we have stopped running tests
	if ( DeviceClusterManager.HasActiveDevice() )
	{
		for (int32 ClusterIndex = 0; ClusterIndex < DeviceClusterManager.GetNumClusters(); ++ClusterIndex)
		{
			//for each device in this cluster
			for (int32 DeviceIndex = 0; DeviceIndex < DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex); ++DeviceIndex)
			{
				//mark the device as idle
				DeviceClusterManager.SetTest(ClusterIndex, DeviceIndex, NULL);

				// Send command to reset tests (delete local files, etc)
				FMessageAddress MessageAddress = DeviceClusterManager.GetDeviceMessageAddress(ClusterIndex, DeviceIndex);

				UE_LOGF(LogAutomationController, Log, "Sending StopTests to %ls", *MessageAddress.ToString());
				SendMessage(
					FMessageEndpoint::MakeMessage<FAutomationWorkerStopTests>(), 
					FAutomationWorkerStopTests::StaticStruct(),
					MessageAddress);
			}
		}

		SetControllerStatus(EAutomationControllerModuleState::Ready);
	}
	else
	{
		SetControllerStatus(EAutomationControllerModuleState::Disabled);
	}

	TestRunningArray.Empty();

	StopStartedTestSessions();

	// Close play window
#if WITH_EDITOR
	if (GUnrealEd && !bKeepPIEOpen)
	{
		GUnrealEd->RequestEndPlayMap();
	}
#endif
}

void FAutomationControllerManager::Init()
{
	extern void EmptyLinkFunctionForStaticInitializationAutomationExecCmd();
	EmptyLinkFunctionForStaticInitializationAutomationExecCmd();

	AutomationTestState = EAutomationControllerModuleState::Disabled;
	bTestResultsAvailable = false;
	bSendAnalytics = FParse::Param(FCommandLine::Get(), TEXT("SendAutomationAnalytics"));
	RequestedTestFlags = EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::PerfFilter;
}

void FAutomationControllerManager::RequestLoadAsset(const FString& InAssetName)
{
#if WITH_EDITOR
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FAssetEditorRequestOpenAsset>(InAssetName), EMessageScope::Process);
#endif
}

void FAutomationControllerManager::Tick()
{
	ProcessAvailableTasks();
	ProcessComparisonQueue();
}

void FAutomationControllerManager::ReportImageComparisonResult(const FAutomationWorkerImageComparisonResults& Result)
{
	// Find the game session instance info
	int32 ClusterIndex;
	int32 DeviceIndex;
	verify(DeviceClusterManager.FindDevice(Result.InstanceId, ClusterIndex, DeviceIndex));

	// Get the current test.
	TSharedPtr<IAutomationReport> Report = DeviceClusterManager.GetTest(ClusterIndex, DeviceIndex);
	if (Report.IsValid())
	{
		// Record the artifacts for the test.
		TMap<FString, FString> LocalFiles;

		FString ScreenshotResultsFolder = FPaths::AutomationReportsDir();

		// Paths in the result are relative to the automation report directory.	
		LocalFiles.Add(ComparisonFileTypes::Unapproved, FPaths::Combine(ScreenshotResultsFolder, Result.ReportIncomingFilePath));

		// unapproved should always be valid; but approved/difference may be empty if this is a new screenshot
		if (Result.ReportComparisonFilePath.Len())
		{
			LocalFiles.Add(ComparisonFileTypes::Difference, FPaths::Combine(ScreenshotResultsFolder, Result.ReportComparisonFilePath));
		}

		// Don't copy reference if the images are similar.
		if (!Result.bSimilar)
		{
			if (Result.ReportIncomingFilePath.Len())
			{
				LocalFiles.Add(ComparisonFileTypes::Approved, FPaths::Combine(ScreenshotResultsFolder, Result.ReportApprovedFilePath));
			}
		}

		Report->AddArtifact(ClusterIndex, CurrentTestPass, FAutomationArtifact(Result.UniqueId, Result.ScreenshotPath, EAutomationArtifactType::Comparison, LocalFiles));
	}
	else
	{
		UE_LOGF(LogAutomationController, Error, "Cannot generate screenshot report for screenshot %ls as report is missing", *Result.IncomingFilePath);
	}
}

void FAutomationControllerManager::ProcessComparisonQueue()
{
	TSharedPtr<FComparisonEntry> Entry;
	if ( ComparisonQueue.Peek(Entry) )
	{
		if ( Entry->PendingComparison.IsReady() )
		{
			const bool Dequeued = ComparisonQueue.Dequeue(Entry);
			check(Dequeued);

			FImageComparisonResult Result = Entry->PendingComparison.Get();
			FAutomationWorkerImageComparisonResults ResultMessage(
				Entry->InstanceId,
				Result.ComparisonId,
				Result.ScreenshotPath,
				Result.IsNew(),
				Result.AreSimilar(),
				Result.MaxLocalDifference,
				Result.GlobalDifference,
				Result.ErrorMessage.ToString(),
				Result.IncomingFilePath,
				Result.ReportComparisonFilePath,
				Result.ReportApprovedFilePath,
				Result.ReportIncomingFilePath
			);

			// Send the message back to the automation worker letting it know the results of the comparison test.
			{
				FAutomationWorkerImageComparisonResults* Message = FMessageEndpoint::MakeMessage<FAutomationWorkerImageComparisonResults>(ResultMessage);

				UE_LOGF(LogAutomationController, Log, "Sending ImageComparisonResult to %ls (IsNew=%d, AreSimilar=%d)",
					*Entry->Sender.ToString()
					, Result.IsNew()
					, Result.AreSimilar()
					);
				SendMessage(Message, Message->StaticStruct(), Entry->Sender);
			}

			if (!Result.bSkipAttachingImages)
			{
				ReportImageComparisonResult(ResultMessage);
			}
		}
	}
}

void FAutomationControllerManager::ProcessAvailableTasks()
{
	// Distribute tasks
	if ( ClusterDistributionMask != 0 )
	{
		// For each device cluster
		for ( int32 ClusterIndex = 0; ClusterIndex < DeviceClusterManager.GetNumClusters(); ++ClusterIndex )
		{
			bool bAllTestsComplete = true;

			// If any of the devices were valid
			if ( ( ClusterDistributionMask & ( 1 << ClusterIndex ) ) && DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex) > 0 )
			{
				ExecuteNextTask(ClusterIndex, bAllTestsComplete);
			}

			//if we're all done running our tests
			if ( bAllTestsComplete )
			{
				//we don't need to test this cluster anymore
				ClusterDistributionMask &= ~( 1 << ClusterIndex );

				if ( ClusterDistributionMask == 0 )
				{
					ProcessResults();

					// Close play window
					#if WITH_EDITOR
					if (GUnrealEd && !bKeepPIEOpen)
					{
						GUnrealEd->RequestEndPlayMap();
					}
					#endif

					// Notify the workers about stopping the session.
					StopStartedTestSessions();

					//Notify the graphical layout we are done processing results.
					TestsCompleteDelegate.Broadcast();
				}
			}
		}
	}

	if ( bIsLocalSession == false )
	{
		// Update the test status for timeouts if this is not a local session
		UpdateTests();
	}
}

void FAutomationControllerManager::ReportTestResults()
{
	UE_LOGF(LogAutomationController, Log, "Test Pass Results:");
	for ( int32 i = 0; i < JsonTestPassResults.Tests.Num(); i++ )
	{
		UE_LOGF(LogAutomationController, Log, "%ls: %ls", *JsonTestPassResults.Tests[i].TestDisplayName, *AutomationStateToString(JsonTestPassResults.Tests[i].State));
	}
}

void FAutomationControllerManager::CollectTestResults(TSharedPtr<IAutomationReport> Report, const FAutomationTestResults& Results)
{
	if (JsonTestPassResults.IsRequired)
	{
		JsonTestPassResults.UpdateTestResultStatus(Report, Results.State, Results.GetWarningTotal() > 0);
		FAutomatedTestResult& TestResult = JsonTestPassResults.GetTestResult(Report);
		TestResult.SetEvents(Results.GetEntries(), Results.GetWarningTotal(), Results.GetErrorTotal());
		TestResult.SetArtifacts(Results.Artifacts);
		TestResult.Duration = Results.Duration;
		
		if (TestResult.DeviceInstance.IsEmpty())
		{
			TestResult.DeviceInstance = { Results.GameInstance };
		}

		JsonTestPassResults.TotalDuration += Results.Duration;

		// Only save the artifacts if there is a valid ReportExportPath
		if (!ReportExportPath.IsEmpty())
		{
			FString ArtifactDirName = TEXT("imageCompare");
			if (FEngineVersion::Current().GetChangelist() != 0)
			{
				ArtifactDirName = ArtifactDirName / FString::FromInt(FEngineVersion::Current().GetChangelist());
			}
			FString ArtifactExportPath = ReportExportPath / ArtifactDirName;

			// Copy new artifacts to Report export path
			FCriticalSection CS;
			for (FAutomationArtifact& Artifact : TestResult.GetArtifacts())
			{
				TArray<FString> Keys;
				Artifact.LocalFiles.GetKeys(Keys);

				ParallelFor(Keys.Num(), [&](int32 Index)
					{
						const FString& Key = Keys[Index];
						FString Path = Artifact.LocalFiles[Key];
						FPaths::MakePathRelativeTo(Path, *FPaths::AutomationReportsDir());
						Path = ArtifactDirName / Path;
						{
							FScopeLock Lock(&CS);
							Artifact.Files.Add(Key, MoveTemp(Path));
						}
						if (Artifact.Type == EAutomationArtifactType::Comparison && Key == ComparisonFileTypes::Unapproved)
						{
							// Trigger the copy of screenshot comparison artifacts only once, using 'Unapproved' key file as cue (since it is always generated)
							bool bOnlyGeneratedFiles = Keys.Num() < 3;
							FScreenshotExportResult ExportResult = ScreenshotManager->ExportScreenshotComparisonResult(Artifact.Name, ArtifactExportPath, bOnlyGeneratedFiles);

							FScopeLock Lock(&CS);
							if (!JsonTestPassResults.ComparisonExported && ExportResult.Success)
							{
								JsonTestPassResults.ComparisonExported = ExportResult.Success;
								JsonTestPassResults.ComparisonExportDirectory = ExportResult.ExportPath;
							}
						}
					});
			}
		}
	}
}

bool FAutomationControllerManager::GenerateJsonTestPassSummary(FAutomatedTestPassResults& SerializedPassResults, FString& OutTestReportJson)
{
	UE_LOGF(LogAutomationController, Display, "Converting results to json object...");
	SerializedPassResults.ReportCreatedOn = FDateTime::UtcNow();

	if (FJsonObjectConverter::UStructToJsonObjectString(SerializedPassResults, OutTestReportJson))
	{
		return true;
	}
	else
	{
		UE_LOGF(LogAutomationController, Error, "Failed to convert test results to json object - No report will be generated.");
	}
		
	return false;
}

bool FAutomationControllerManager::SaveJsonTestPassSummary(const FString& TestReportJson)
{
	if (!TestReportJson.IsEmpty())
	{
		FString ReportFileName = FString::Printf(TEXT("%s/index.json"), *ReportExportPath);
		const int32 WriteAttempts = 3;
		const float SleepBetweenAttempts = 0.05f;

		for (int32 Attempt = 1; Attempt <= WriteAttempts; ++Attempt)
		{
			if (FFileHelper::SaveStringToFile(TestReportJson, *ReportFileName, FFileHelper::EEncodingOptions::ForceUTF8))
			{
				UE_LOGF(LogAutomationController, Display, "Successfully wrote json results file!");
				return true;
			}

			FPlatformProcess::Sleep(SleepBetweenAttempts);
		}

		UE_LOGF(LogAutomationController, Warning, "Failed to write test report json to '%ls' after 3 attempts - No report will be generated.", *ReportFileName);
	}
	else
	{
		UE_LOGF(LogAutomationController, Error, "Json is empty. No test results report has been generated.");
	}

	return false;
}

void FAutomationControllerManager::BroadcastJsonTestPassSummary(const FString& TestReportJson)
{
	if (!TestReportGeneratedDelegate.IsBound())
	{
		return;
	}

	if (!TestReportJson.IsEmpty())
	{
		TestReportGeneratedDelegate.Broadcast(TestReportJson);
	}
	else
	{
		UE_LOGF(LogAutomationController, Error, "Json is empty. No test results report has been broadcast.");
	}
}

bool FAutomationControllerManager::GenerateTestPassHtmlIndex()
{
	UE_LOGF(LogAutomationController, Display, "Loading results html template...");

	FString ReportTemplate;
	if (FFileHelper::LoadFileToString(ReportTemplate, *(FPaths::EngineContentDir() / TEXT("Automation/Report-Template.html"))))
	{		
		FString ReportFileName = FString::Printf(TEXT("%s/index.html"), *ReportExportPath);
		const int32 WriteAttempts = 3;
		const float SleepBetweenAttempts = 0.05f;

		for (int32 Attempt = 1; Attempt <= WriteAttempts; ++Attempt)
		{
			if (FFileHelper::SaveStringToFile(ReportTemplate, *ReportFileName, FFileHelper::EEncodingOptions::ForceUTF8))
			{
				UE_LOGF(LogAutomationController, Display, "Successfully wrote html results file!");
				return true;
			}

			FPlatformProcess::Sleep(SleepBetweenAttempts);
		}

		UE_LOGF(LogAutomationController, Warning, "Failed to write test report html to '%ls' after 3 attempts - No report will be generated.", *ReportFileName);
	}
	else
	{
		UE_LOGF(LogAutomationController, Error, "Failed to load test report html template - No report will be generated.");
	}
	
	return false;
}

bool FAutomationControllerManager::LoadJsonTestPassSummary(FString& ReportFilePath, TArray<IAutomationReportPtr> TestReports)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	FString Json;
	if (!FFileHelper::LoadFileToString(Json, *ReportFilePath))
	{
		UE_LOGF(LogAutomationController, Error, "Failed to read json results file '%ls'.", *ReportFilePath);
		return false;
	}
	if (!FJsonObjectConverter::JsonObjectStringToUStruct(Json, &JsonTestPassResults))
	{
		UE_LOGF(LogAutomationController, Error, "Failed to load json results file '%ls'.", *ReportFilePath);
		return false;
	}
	UE_LOGF(LogAutomationController, Display, "Successfully loaded json results file.");

	JsonTestPassResults.ReBuildTestsMapIndex();

	// Reflect Test Result Status on Report Status
	for (IAutomationReportPtr& TestReport : TestReports)
	{
		if (!JsonTestPassResults.ReflectResultStateToReport(TestReport))
		{
			UE_LOGF(LogAutomationController, Warning, "No match to test '%ls' in json results.", *TestReport->GetFullTestPath());
		}
	}

	return true;
}

FString FAutomationControllerManager::SlugString(const FString& DisplayString) const
{
	FString GeneratedName = DisplayString;

	// Convert the display label, which may consist of just about any possible character, into a
	// suitable name for a UObject (remove whitespace, certain symbols, etc.)
	{
		for ( int32 BadCharacterIndex = 0; BadCharacterIndex < UE_ARRAY_COUNT(INVALID_OBJECTNAME_CHARACTERS) - 1; ++BadCharacterIndex )
		{
			const TCHAR TestChar[2] = { INVALID_OBJECTNAME_CHARACTERS[BadCharacterIndex], 0 };
			const int32 NumReplacedChars = GeneratedName.ReplaceInline(TestChar, TEXT(""));
		}
	}

	return GeneratedName;
}

FString FAutomationControllerManager::GetReportOutputPath() const
{
	return ReportExportPath;
}

void FAutomationControllerManager::ExecuteNextTask( int32 ClusterIndex, OUT bool& bAllTestsCompleted )
{
	bool bTestThatRequiresMultiplePraticipantsHadEnoughParticipants = false;
	TArray< IAutomationReportPtr > TestsRunThisPass;

	// For each device in this cluster
	int32 NumDevicesInCluster = DeviceClusterManager.GetNumDevicesInCluster( ClusterIndex );
	for ( int32 DeviceIndex = 0; DeviceIndex < NumDevicesInCluster; ++DeviceIndex )
	{
		// If this device is idle
		if ( !DeviceClusterManager.GetTest(ClusterIndex, DeviceIndex).IsValid() && DeviceClusterManager.DeviceEnabled(ClusterIndex, DeviceIndex) )
		{
			// Get the next test that should be worked on
			TSharedPtr< IAutomationReport > NextTest = ReportManager.GetNextReportToExecute(bAllTestsCompleted, ClusterIndex, CurrentTestPass, NumDevicesInCluster);
			if ( NextTest.IsValid() )
			{
				// Get the status of the test
				EAutomationState TestState = NextTest->GetState(ClusterIndex, CurrentTestPass);
				if (TestState == EAutomationState::NotRun)
				{
					// Reserve this device for the test
					DeviceClusterManager.SetTest(ClusterIndex, DeviceIndex, NextTest);
					TestsRunThisPass.Add(NextTest);

					// If we now have enough devices reserved for the test, run it!
					TArray<FMessageAddress> DeviceAddresses = DeviceClusterManager.GetDevicesReservedForTest(ClusterIndex, NextTest);
					if (DeviceAddresses.Num() == NextTest->GetNumParticipantsRequired())
					{
						TArray<FString> GameInstances;

						// Send it to each device
						for (int32 AddressIndex = 0; AddressIndex < DeviceAddresses.Num(); ++AddressIndex)
						{
							const FAutomationDeviceInfo& DeviceInfo = DeviceClusterManager.GetDeviceInfo(ClusterIndex, DeviceIndex);
							FAutomationTestResults TestResults;
							TestResults.GameInstance = DeviceInfo.Instance.ToString();
							TestResults.State = EAutomationState::InProcess;
							GameInstances.Add(TestResults.GameInstance);
							NextTest->SetResults(ClusterIndex, CurrentTestPass, TestResults);

							// Mark the device as busy
							FMessageAddress DeviceAddress = DeviceAddresses[AddressIndex];

							// Send the test to the device for execution!
							UE_LOGF(LogAutomationController, Log, "Sending RunTest %ls to %ls", *NextTest->GetDisplayName(), *DeviceAddress.ToString());

							SendMessage(
								FMessageEndpoint::MakeMessage<FAutomationWorkerRunTests>(ExecutionCount, AddressIndex, NextTest->GetCommand(), NextTest->GetDisplayName(), NextTest->GetFullTestPath(), bSendAnalytics, bPruneLogsOnSuccess),
								FAutomationWorkerRunTests::StaticStruct(),
								DeviceAddress);

							// Add a test so we can check later if the device is still active
							TestRunningArray.Add(FTestRunningInfo(DeviceAddress, DeviceInfo.Instance));
						}

						UE_LOG(LogAutomationController, Display, AutomationTestStarting, *NextTest->GetDisplayName(), *NextTest->GetFullTestPath());

						if (JsonTestPassResults.IsRequired)
						{
							JsonTestPassResults.UpdateTestResultStatus(NextTest, EAutomationState::InProcess);
							FAutomatedTestResult& TestResult = JsonTestPassResults.GetTestResult(NextTest);
							TestResult.DeviceInstance = GameInstances;
							if (!ReportExportPath.IsEmpty() && bResumeRunTest)
							{
								// Save the whole pass report so that if the next test triggers a critical failure we are not left with no pass report and we can resume.
								FString Json;
								if (GenerateJsonTestPassSummary(JsonTestPassResults, Json))
								{
									SaveJsonTestPassSummary(Json);
								}
							}
						}

						NextTest->ResetNetworkCommandResponses();
					}
				}
			}
		}
		else
		{
			// At least one device is still working
			bAllTestsCompleted = false;
		}
	}

	// Ensure any tests we have attempted to run on this pass had enough participants to successfully run.
	for ( int32 TestIndex = 0; TestIndex < TestsRunThisPass.Num(); TestIndex++ )
	{
		IAutomationReportPtr CurrentTest = TestsRunThisPass[TestIndex];

		if ( CurrentTest->GetNumDevicesRunningTest() != CurrentTest->GetNumParticipantsRequired() )
		{
			if ( GetNumDevicesInCluster(ClusterIndex) < CurrentTest->GetNumParticipantsRequired() )
			{
				FAutomationTestResults TestResults;
				TestResults.State = EAutomationState::Skipped;
				TestResults.AddEvent(FAutomationEvent(EAutomationEventType::Warning, FString::Printf(TEXT("Skipped because the test needed %d devices to participate, Only had %d available."), CurrentTest->GetNumParticipantsRequired(), DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex))));

				CurrentTest->SetResults(ClusterIndex, CurrentTestPass, TestResults);
				CollectTestResults(CurrentTest, TestResults);
				DeviceClusterManager.ResetAllDevicesRunningTest(ClusterIndex, CurrentTest);
			}
		}
	}

	//Check to see if we finished a pass
	if ( bAllTestsCompleted && CurrentTestPass < NumTestPasses - 1 )
	{
		CurrentTestPass++;
		ReportManager.SetCurrentTestPass(CurrentTestPass);
		bAllTestsCompleted = false;
	}
}


void FAutomationControllerManager::Startup()
{
	MessageEndpoint = FMessageEndpoint::Builder("FAutomationControllerModule")
		.Handling<FAutomationWorkerFindWorkersResponse>(this, &FAutomationControllerManager::HandleFindWorkersResponseMessage)
		.Handling<FAutomationWorkerPong>(this, &FAutomationControllerManager::HandlePongMessage)
		.Handling<FAutomationWorkerRequestNextNetworkCommand>(this, &FAutomationControllerManager::HandleRequestNextNetworkCommandMessage)
		.Handling<FAutomationWorkerRequestTestsReplyComplete>(this, &FAutomationControllerManager::HandleRequestTestsReplyCompleteMessage)
		.Handling<FAutomationWorkerRunTestsReply>(this, &FAutomationControllerManager::HandleRunTestsReplyMessage)
		.Handling<FAutomationWorkerScreenImage>(this, &FAutomationControllerManager::HandleReceivedScreenShot)
		.Handling<FAutomationWorkerImageComparisonResults>(this, &FAutomationControllerManager::HandleReceivedComparisonResult)
		.Handling<FAutomationWorkerTestDataRequest>(this, &FAutomationControllerManager::HandleTestDataRequest)
		.Handling<FAutomationWorkerWorkerOffline>(this, &FAutomationControllerManager::HandleWorkerOfflineMessage)
		.Handling<FAutomationWorkerTelemetryData>(this, &FAutomationControllerManager::HandleReceivedTelemetryData);

	if ( MessageEndpoint.IsValid() )
	{
		MessageEndpoint->Subscribe<FAutomationWorkerWorkerOffline>();
	}

	ClusterDistributionMask = 0;
	ExecutionCount = 0;
	bDeveloperDirectoryIncluded = false;
	RequestedTestFlags = EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::PerfFilter;

	NumTestPasses = 1;

	//Default to machine name
	DeviceGroupFlags = 0;
	ToggleDeviceGroupFlag(EAutomationDeviceGroupTypes::MachineName);
}

void FAutomationControllerManager::Shutdown()
{
	MessageEndpoint.Reset();
	ShutdownDelegate.Broadcast();
	RemoveCallbacks();
}

void FAutomationControllerManager::RemoveCallbacks()
{
	ShutdownDelegate.Clear();
	TestsAvailableDelegate.Clear();
	TestsRefreshedDelegate.Clear();
	TestsCompleteDelegate.Clear();
	TestReportGeneratedDelegate.Clear();
}

void FAutomationControllerManager::SetTestNames(const FGuid& AutomationWorkerInstanceId, TArray<FAutomationTestInfo>& TestInfo)
{
	int32 DeviceClusterIndex = INDEX_NONE;
	int32 DeviceIndex = INDEX_NONE;

	// Find the device that requested these tests
	if ( DeviceClusterManager.FindDevice(AutomationWorkerInstanceId, DeviceClusterIndex, DeviceIndex) )
	{
		// Sort tests by display name
		struct FCompareAutomationTestInfo
		{
			FORCEINLINE bool operator()(const FAutomationTestInfo& A, const FAutomationTestInfo& B) const
			{
				return A.GetDisplayName() < B.GetDisplayName();
			}
		};

		TestInfo.Sort(FCompareAutomationTestInfo());

		// Add each test to the collection
		for ( int32 TestIndex = 0; TestIndex < TestInfo.Num(); ++TestIndex )
		{
			// Ensure our test exists. If not, add it
			ReportManager.EnsureReportExists(TestInfo[TestIndex], DeviceClusterIndex, NumTestPasses);
		}
	}
	else
	{
		//todo automation - make sure to report error if the device was not discovered correctly
	}

	// Note the response
	RefreshTestResponses++;

	// If we have received all the responses we expect to
	if ( RefreshTestResponses == DeviceClusterManager.GetNumClusters() )
	{
		TestsRefreshedDelegate.Broadcast();
	}
}

void FAutomationControllerManager::ProcessResults()
{
	bHasErrors = false;
	bHasWarning = false;
	bHasLogs = false;

	TArray< TSharedPtr< IAutomationReport > > TestReports = GetEnabledReports();

	if ( TestReports.Num() )
	{
		bTestResultsAvailable = true;

		for ( int32 Index = 0; Index < TestReports.Num(); Index++ )
		{
			CheckChildResult(TestReports[Index]);
		}
	}

	if ( JsonTestPassResults.IsRequired )
	{
		UE_LOGF(LogAutomationController, Display, "Exporting Automation Report.");

		FAutomatedTestPassResults SerializedPassResults = JsonTestPassResults;

		if (bSortTestsByFailure)
		{
			// Sort result by failure to improve readability (disabled by default)
			SerializedPassResults.Tests.StableSort([](const FAutomatedTestResult& A, const FAutomatedTestResult& B) {
				if (A.GetErrorTotal() > 0)
				{
					if (B.GetErrorTotal() > 0)
						return (A.FullTestPath < B.FullTestPath);
					else
						return true;
				}
				else if (B.GetErrorTotal() > 0)
				{
					return false;
				}

				if (A.State == EAutomationState::Skipped)
				{
					if (B.State == EAutomationState::Skipped)
						return  (A.FullTestPath < B.FullTestPath);
					else
						return true;
				}
				else if (B.State == EAutomationState::Skipped)
				{
					return false;
				}

				if (A.GetWarningTotal() > 0)
				{
					if (B.GetWarningTotal() > 0)
						return (A.FullTestPath < B.FullTestPath);
					else
						return true;
				}
				else if (B.GetWarningTotal() > 0)
				{
					return false;
				}

				return A.FullTestPath < B.FullTestPath;
			});
		}

		if (bRequiresReportGeneration)
		{
			// Generate Json
			FString Json;
			if (GenerateJsonTestPassSummary(SerializedPassResults, Json))
			{
				if (!ReportExportPath.IsEmpty())
				{
					FDateTime StepTime = FDateTime::Now();
					UE_LOGF(LogAutomationController, Log, "Writing reports to %ls...", *ReportExportPath);

					SaveJsonTestPassSummary(Json);
					UE_LOGF(LogAutomationController, Log, "Exported report to '%ls' in %.02f Seconds", *ReportExportPath, (FDateTime::Now() - StepTime).GetTotalSeconds());
				}

				if (TestReportGeneratedDelegate.IsBound())
				{
					BroadcastJsonTestPassSummary(Json);
				}
			}
			else
			{
				UE_LOGF(LogAutomationController, Error, "Failed to serialize test results into json. No test results report can be generated.");
			}
		}

		if (!ReportURLPath.IsEmpty())
		{
			UE_LOGF(LogAutomationController, Display, "Report can be viewed in a browser at '%ls'", DeveloperReportUrl.IsEmpty() ? *ReportURLPath : *DeveloperReportUrl);

			if (!DeveloperReportUrl.IsEmpty())
			{
				UE_LOGF(LogAutomationController, Display, "Launching Report URL %ls.", *DeveloperReportUrl);

				FPlatformProcess::LaunchURL(*DeveloperReportUrl, nullptr, nullptr);
			}
		}

		UE_LOGF(LogAutomationController, Display, "Report can be opened in the editor at '%ls'", ReportExportPath.IsEmpty() ? *FPaths::ConvertRelativePathToFull(FPaths::AutomationReportsDir()) : *ReportExportPath);
	}

	// Then clean our array for the next pass.
	JsonTestPassResults.ClearAllEntries();

	SetControllerStatus(EAutomationControllerModuleState::Ready);
}

void FAutomationControllerManager::CheckChildResult(TSharedPtr<IAutomationReport> InReport)
{
	TArray<TSharedPtr<IAutomationReport> >& ChildReports = InReport->GetChildReports();

	if ( ChildReports.Num() > 0 )
	{
		for ( int32 Index = 0; Index < ChildReports.Num(); Index++ )
		{
			CheckChildResult(ChildReports[Index]);
		}
	}
	else if ( ( bHasErrors && bHasWarning && bHasLogs ) == false && InReport->IsEnabled() )
	{
		for ( int32 ClusterIndex = 0; ClusterIndex < GetNumDeviceClusters(); ++ClusterIndex )
		{
			FAutomationTestResults TestResults = InReport->GetResults(ClusterIndex, CurrentTestPass);

			if ( TestResults.GetErrorTotal() > 0 )
			{
				bHasErrors = true;
			}
			if ( TestResults.GetWarningTotal() )
			{
				bHasWarning = true;
			}
			if ( TestResults.GetLogTotal() )
			{
				bHasLogs = true;
			}
		}
	}
}

void FAutomationControllerManager::SetControllerStatus(EAutomationControllerModuleState::Type InAutomationTestState)
{
	if ( InAutomationTestState != AutomationTestState )
	{
		// Inform the UI if the test state has changed
		AutomationTestState = InAutomationTestState;
		TestsAvailableDelegate.Broadcast(AutomationTestState);
	}
}

void FAutomationControllerManager::RemoveTestRunning(const FGuid& OwnerInstanceId)
{
	for ( int32 Index = 0; Index < TestRunningArray.Num(); Index++ )
	{
		if ( TestRunningArray[Index].OwnerInstanceId == OwnerInstanceId )
		{
			TestRunningArray.RemoveAt(Index);
			break;
		}
	}
}

void FAutomationControllerManager::AddPingResult(const FGuid& ResponderInstanceId)
{
	for ( int32 Index = 0; Index < TestRunningArray.Num(); Index++ )
	{
		if (TestRunningArray[Index].OwnerInstanceId == ResponderInstanceId)
		{
			TestRunningArray[Index].LastPingTime = 0;
			break;
		}
	}
}

void FAutomationControllerManager::UpdateTests()
{
	const double kIgnoreAsFrameHitchDelta = 2.0f;
	const double TickDelta = FPlatformTime::Seconds() - LastTimeUpdateTicked;

	LastTimeUpdateTicked = FPlatformTime::Seconds();

	// If this tick was very long then don't check the health of tests. If we've been blocked for a while on a load and ping responses haven't yet been processed
	// then otherwise might add a delta to the ping time that causes the test to look like it's timed out.
	if (TickDelta < kIgnoreAsFrameHitchDelta)
	{
		CheckTestTimer += TickDelta;
		if (CheckTestTimer > CheckTestIntervalSeconds)
		{
			for ( int32 Index = 0; Index < TestRunningArray.Num(); Index++ )
			{
				TestRunningArray[Index].LastPingTime += CheckTestTimer;

				if (TestRunningArray[Index].LastPingTime > GameInstanceLostTimerSeconds)
				{
					// Find the game session instance info
					int32 ClusterIndex;
					int32 DeviceIndex;
					verify(DeviceClusterManager.FindDevice(TestRunningArray[Index].OwnerInstanceId, ClusterIndex, DeviceIndex));
					//verify this device thought it was busy
					TSharedPtr <IAutomationReport> Report = DeviceClusterManager.GetTest(ClusterIndex, DeviceIndex);
					check(Report.IsValid());

					bHasErrors = true;

					FAutomationTestResults TestResults;
					TestResults.State = EAutomationState::Fail;
					TestResults.GameInstance = DeviceClusterManager.GetClusterGameInstanceId(ClusterIndex, DeviceIndex).ToString();
					FString DeviceName = DeviceClusterManager.GetClusterDeviceName(ClusterIndex, DeviceIndex);
					TestResults.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("Timeout waiting for device %s"), *DeviceName)));

					UE_LOGF(LogAutomationController, Error, "Removing test and marking failure after timeout waiting for device %ls LastPingTime was greater than %.1f", *DeviceName, GameInstanceLostTimerSeconds);

					// Set the results
					Report->SetResults(ClusterIndex, CurrentTestPass, TestResults);
					bTestResultsAvailable = true;

					const FAutomationTestResults& FinalResults = Report->GetResults(ClusterIndex, CurrentTestPass);

					// Gather all of the data relevant to this test for our json reporting.
					CollectTestResults(Report, FinalResults);

					// Disable the device in the cluster so it is not used again
					DeviceClusterManager.DisableDevice(ClusterIndex, DeviceIndex);

					// Remove the running test
					TestRunningArray.RemoveAt(Index--);

					// If there are no more devices, set the module state to disabled
					if ( DeviceClusterManager.HasActiveDevice() == false )
					{
						// Process results first to write out the report
						ProcessResults();

						UE_LOGF(LogAutomationController, Display, "Module disabled");
						SetControllerStatus(EAutomationControllerModuleState::Disabled);
						ClusterDistributionMask = 0;
					}
					else
					{
						UE_LOGF(LogAutomationController, Display, "Module not disabled. Keep looking.");
						// Remove the cluster from the mask if there are no active devices left
						if ( DeviceClusterManager.GetNumActiveDevicesInCluster(ClusterIndex) == 0 )
						{
							ClusterDistributionMask &= ~( 1 << ClusterIndex );
						}
						if ( TestRunningArray.Num() == 0 )
						{
							SetControllerStatus(EAutomationControllerModuleState::Ready);
						}
					}
				}
				else
				{
					SendMessage(
						FMessageEndpoint::MakeMessage<FAutomationWorkerPing>(),
						FAutomationWorkerPing::StaticStruct(),
						TestRunningArray[Index].OwnerMessageAddress);
				}
			}
			CheckTestTimer = 0.f;
		}
	}
	else
	{
		UE_LOGF(LogAutomationController, Log, "Ignoring very large delta of %.02f seconds in calls to FAutomationControllerManager::Tick() and not penalizing unresponsive tests", TickDelta);
	}
}

void FAutomationControllerManager::StopStartedTestSessions()
{
	int32 ClusterIndex = INDEX_NONE;
	int32 DeviceIndex = INDEX_NONE;

	// Notify the workers about stopping the session.
	for (const auto& WorkerInstanceId : StartedTestSessionWorkerInstanceIdSet)
	{
		if (DeviceClusterManager.FindDevice(WorkerInstanceId, ClusterIndex, DeviceIndex))
		{
			const FMessageAddress MessageAddress = DeviceClusterManager.GetDeviceMessageAddress(ClusterIndex, DeviceIndex);
			UE_LOGF(LogAutomationController, Log, "Sending StopTestSession to %ls", *MessageAddress.ToString());
			SendMessage(
				FMessageEndpoint::MakeMessage<FAutomationWorkerStopTestSession>(),
				FAutomationWorkerStopTestSession::StaticStruct(),
				MessageAddress);
		}
	}
}

void FAutomationControllerManager::SendMessage(FAutomationWorkerMessageBase* Message, UScriptStruct* TypeInfo, const FMessageAddress& ControllerAddress)
{
	check(nullptr != Message);

	Message->InstanceId = FApp::GetInstanceId();
	MessageEndpoint->Send(
		Message,
		TypeInfo,
		EMessageFlags::None,
		nullptr,
		TArrayBuilder<FMessageAddress>().Add(ControllerAddress),
		FTimespan::Zero(),
		FDateTime::MaxValue());
}

const bool FAutomationControllerManager::ExportReport(uint32 FileExportTypeMask)
{
	return ReportManager.ExportReport(FileExportTypeMask, GetNumDeviceClusters());
}

bool FAutomationControllerManager::IsTestRunnable(IAutomationReportPtr InReport) const
{
	bool bIsRunnable = false;

	for ( int32 ClusterIndex = 0; ClusterIndex < GetNumDeviceClusters(); ++ClusterIndex )
	{
		if ( InReport->IsSupported(ClusterIndex) )
		{
			if ( GetNumDevicesInCluster(ClusterIndex) >= InReport->GetNumParticipantsRequired() )
			{
				bIsRunnable = true;
				break;
			}
		}
	}

	return bIsRunnable;
}

/* FAutomationControllerModule callbacks
 *****************************************************************************/

void FAutomationControllerManager::HandleFindWorkersResponseMessage(const FAutomationWorkerFindWorkersResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationController, Log, "Received FindWorkersResponseMessage from %ls", *Context->GetSender().ToString());

	if ( Message.SessionId == ActiveSessionId )
	{
		DeviceClusterManager.AddDeviceFromMessage(Context->GetSender(), Message, DeviceGroupFlags);
	}

	if (AutomationTestState == EAutomationControllerModuleState::Running)
	{
		UE_LOGF(LogAutomationController, Verbose, "Already running test. Don't request tests from %ls", *Context->GetSender().ToString());
		return;
	}

	RequestTests();

	SetControllerStatus(EAutomationControllerModuleState::Ready);
}

void FAutomationControllerManager::HandlePongMessage( const FAutomationWorkerPong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);
	AddPingResult(Message.InstanceId);
}

void FAutomationControllerManager::HandleReceivedScreenShot(const FAutomationWorkerScreenImage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationController, Log, "ReceivedScreenShot for %ls (%dx%d) from %ls", *Message.Metadata.TestName, Message.Metadata.Width, Message.Metadata.Height, *Context->GetSender().ToString());
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);

	bool bTree = true;

	FString OutputSubFolder = FPaths::Combine(Message.Metadata.Context, Message.Metadata.ScreenShotName);
	FString OutputFolder = FPaths::Combine(FPaths::AutomationTransientDir(), FPaths::GetPath(Message.ScreenShotName));

	FString IncomingFileName = FPaths::Combine(OutputFolder,FPaths::GetCleanFilename(Message.ScreenShotName));
	FString TraceFileName = FPaths::ChangeExtension(IncomingFileName, TEXT(".rdc"));

	FString DirectoryPath = FPaths::GetPath(IncomingFileName);

	if (!IFileManager::Get().MakeDirectory(*DirectoryPath, bTree))
	{
		UE_LOGF(LogAutomationController, Error, "Failed to create directory %ls for incoming screenshot", *DirectoryPath);
		return;
	}
	
	if (!FFileHelper::SaveArrayToFile(Message.ScreenImage, *IncomingFileName))
	{
		uint32 WriteErrorCode = FPlatformMisc::GetLastError();
		TCHAR WriteErrorBuffer[2048];
		FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
		UE_LOGF(LogAutomationController, Warning, "Fail to save screenshot to %ls. WriteError: %u (%ls)", *IncomingFileName, WriteErrorCode, WriteErrorBuffer);
		return;
	}

	if (Message.FrameTrace.Num() > 0)
	{
		if (!FFileHelper::SaveArrayToFile(Message.FrameTrace, *TraceFileName))
		{
			uint32 WriteErrorCode = FPlatformMisc::GetLastError();
			TCHAR WriteErrorBuffer[2048];
			FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
			UE_LOGF(LogAutomationController, Warning, "Faild to save frame trace to %ls. WriteError: %u (%ls)", *TraceFileName, WriteErrorCode, WriteErrorBuffer);
		}
	}

	// TODO Automation There is identical code in, Engine\Source\Runtime\AutomationWorker\Private\AutomationWorkerModule.cpp,
	// need to move this code into common area.

	FString Json;
	if ( FJsonObjectConverter::UStructToJsonObjectString(Message.Metadata, Json) )
	{
		FString MetadataPath = FPaths::ChangeExtension(IncomingFileName, TEXT("json"));
		FFileHelper::SaveStringToFile(Json, *MetadataPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
	}

	// compare the incoming image and throw it away afterwards (note - there will be a copy in the report)
	TSharedRef<FComparisonEntry> Comparison = MakeShareable(new FComparisonEntry());
	Comparison->Sender = Context->GetSender();
	Comparison->InstanceId = Message.InstanceId;
	Comparison->PendingComparison = ScreenshotManager->CompareScreenshotAsync(IncomingFileName, Message.Metadata, EScreenShotCompareOptions::DiscardImage);

	ComparisonQueue.Enqueue(Comparison);
}

void FAutomationControllerManager::HandleReceivedComparisonResult(const FAutomationWorkerImageComparisonResults& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);
	ReportImageComparisonResult(Message);
}

void FAutomationControllerManager::HandleTestDataRequest(const FAutomationWorkerTestDataRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationController, Log, "Received TestDataRequest from %ls", *Context->GetSender().ToString());
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);

	const FString TestDataRoot = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / TEXT("Test"));
	const FString DataFile = Message.DataType / Message.DataPlatform / Message.DataTestName / Message.DataName + TEXT(".json");
	const FString DataFullPath = TestDataRoot / DataFile;

	// Generate the folder for the data if it doesn't exist.
	const bool bTree = true;
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(DataFile), bTree);

	bool bIsNew = true;
	FString ResponseJsonData = Message.JsonData;

	if ( FPaths::FileExists(DataFullPath) )
	{
		if ( FFileHelper::LoadFileToString(ResponseJsonData, *DataFullPath) )
		{
			bIsNew = false;
		}
		else
		{
			// TODO Error
		}
	}

	if ( bIsNew )
	{

		FString IncomingTestData = FPaths::Combine(FPaths::AutomationTransientDir(), TEXT("Data/"), DataFile);

		if ( FFileHelper::SaveStringToFile(Message.JsonData, *IncomingTestData) )
		{
			//TODO Anything extra to do here?
		}
		else
		{
			//TODO What do we do if this fails?
		}
	}

	FAutomationWorkerTestDataResponse* ResponseMessage = FMessageEndpoint::MakeMessage<FAutomationWorkerTestDataResponse>();
	ResponseMessage->bIsNew = bIsNew;
	ResponseMessage->JsonData = ResponseJsonData;

	SendMessage(ResponseMessage, ResponseMessage->StaticStruct(), Context->GetSender());
}

void FAutomationControllerManager::HandlePerformanceDataRequest(const FAutomationWorkerPerformanceDataRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	//TODO Read/Performance data.
	UE_LOGF(LogAutomationController, Log, "Received PerformanceDataRequest from %ls", *Context->GetSender().ToString());

	FAutomationWorkerPerformanceDataResponse* ResponseMessage = FMessageEndpoint::MakeMessage<FAutomationWorkerPerformanceDataResponse>();
	ResponseMessage->bSuccess = true;
	ResponseMessage->ErrorMessage = TEXT("");

	SendMessage(ResponseMessage, ResponseMessage->StaticStruct(), Context->GetSender());
}

void FAutomationControllerManager::HandleRequestNextNetworkCommandMessage(const FAutomationWorkerRequestNextNetworkCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationController, Log, "Received RequestNextNetworkCommandMessage from %ls", *Context->GetSender().ToString());
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);

	// Harvest iteration of running the tests this result came from (stops stale results from being committed to subsequent runs)
	if ( Message.ExecutionCount == ExecutionCount )
	{
		// Find the device id for the address
		int32 ClusterIndex;
		int32 DeviceIndex;

		verify(DeviceClusterManager.FindDevice(Message.InstanceId, ClusterIndex, DeviceIndex));

		// Verify this device thought it was busy
		TSharedPtr<IAutomationReport> Report = DeviceClusterManager.GetTest(ClusterIndex, DeviceIndex);
		check(Report.IsValid());

		// Increment network command responses
		bool bAllResponsesReceived = Report->IncrementNetworkCommandResponses();

		// Test if we've accumulated all responses AND this was the result for the round of test running AND we're still running tests
		if ( bAllResponsesReceived && ( ClusterDistributionMask & ( 1 << ClusterIndex ) ) )
		{
			// Reset the counter
			Report->ResetNetworkCommandResponses();

			// For every device in this networked test
			TArray<FMessageAddress> DeviceAddresses = DeviceClusterManager.GetDevicesReservedForTest(ClusterIndex, Report);
			check(DeviceAddresses.Num() == Report->GetNumParticipantsRequired());

			// Send it to each device
			for ( int32 AddressIndex = 0; AddressIndex < DeviceAddresses.Num(); ++AddressIndex )
			{
				//send "next command message" to worker
				SendMessage(
					FMessageEndpoint::MakeMessage<FAutomationWorkerNextNetworkCommandReply>(),
					FAutomationWorkerNextNetworkCommandReply::StaticStruct(),
					DeviceAddresses[AddressIndex]	);
			}
		}
	}
}

void FAutomationControllerManager::HandleRequestTestsReplyCompleteMessage(const FAutomationWorkerRequestTestsReplyComplete& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogAutomationController, Log, "Received RequestTestsReplyCompleteMessage from %ls", *Context->GetSender().ToString());
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);

	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Controller"));
	TArray<FAutomationTestInfo> TestInfo;
	TestInfo.Reset(Message.Tests.Num());
	for (const FAutomationWorkerSingleTestReply& SingleTestReply : Message.Tests)
	{
		FAutomationTestInfo NewTest = SingleTestReply.GetTestInfo();
		TestInfo.Add(NewTest);
	}

	UE_LOGF(LogAutomationController, Log, "%d tests available on %ls", TestInfo.Num(), *Context->GetSender().ToString());

	SetTestNames(Message.InstanceId, TestInfo);
}

void FAutomationControllerManager::HandleReceivedTelemetryData(const FAutomationWorkerTelemetryData& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);
	FAutomationTelemetry::HandleAddTelemetry(Message);
}

void FAutomationControllerManager::ReportAutomationResult(const TSharedPtr<IAutomationReport> InReport, int32 ClusterIndex, int32 PassIndex)
{

	FName CategoryName = "LogAutomationController";
#if WITH_EDITOR
	FMessageLog AutomationEditorLog("AutomationTestingLog");
	// we log these messages ourselves for non-editor platforms so suppress this.
	AutomationEditorLog.SuppressLoggingToOutputLog(true);
	AutomationEditorLog.Open();
#endif

	const FAutomationTestResults& Results = InReport->GetResults(ClusterIndex, PassIndex);
	const FString StateString = AutomationStateToString(Results.State);

	// write results to editor panel
#if WITH_EDITOR
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TestName"), FText::FromString(InReport->GetFullTestPath()));
	Arguments.Add(TEXT("Result"), FText::FromString(StateString));
	TSharedRef<FTextToken> Token = FTextToken::Create(FText::Format(LOCTEXT("TestSuccessOrFailure", "Test '{TestName}' completed with result '{Result}'"), Arguments));
	
	if (Results.State == EAutomationState::Success)
	{
		AutomationEditorLog.Info()->AddToken(Token);
	}
	else if (Results.State == EAutomationState::Skipped)
	{
		AutomationEditorLog.Warning()->AddToken(Token);
	}
	else
	{
		AutomationEditorLog.Error()->AddToken(Token);
	}
#endif

	// Now log
	FString ResultString = FString::Printf(AutomationStateFormat, *StateString, *InReport->GetDisplayName(), *InReport->GetFullTestPath());
	if (Results.State == EAutomationState::Fail)
	{
		UE_LOGF(LogAutomationController, Error, "%ls", *ResultString);
	}
	else
	{
		UE_LOGF(LogAutomationController, Display, "%ls", *ResultString);
	}

	// bracket these for easy parsing
	UE_LOG(LogAutomationController, Log, BeginEventsFormat, *InReport->GetFullTestPath());

	for (const FAutomationExecutionEntry& Entry : Results.GetEntries())
	{
#if WITH_EDITOR
		TSharedRef<FTextToken> TextToken = FTextToken::Create(FText::FromString(Entry.ToStringFormattedEditorLog()), false);
#endif
		switch (Entry.Event.Type)
		{
		case EAutomationEventType::Info:
			UE_LOGF(LogAutomationController, Log, "%ls", *Entry.ToString());
#if WITH_EDITOR
			AutomationEditorLog.Info()->AddToken(TextToken);
#endif
			break;
		case EAutomationEventType::Warning:
			UE_LOGF(LogAutomationController, Warning, "%ls", *Entry.ToString());
#if WITH_EDITOR
			AutomationEditorLog.Warning()->AddToken(TextToken);
#endif
			break;
		case EAutomationEventType::Error:
			UE_LOGF(LogAutomationController, Error, "%ls", *Entry.ToString());
#if WITH_EDITOR
			AutomationEditorLog.Error()->AddToken(TextToken);
#endif
			break;
		}
	}

	UE_LOG(LogAutomationController, Log, EndEventsFormat, *InReport->GetFullTestPath());

	#undef AutomationSuccessFormat
	#undef AutomationFailureFormat
	#undef BeginEventsFormat
	#undef EndEventsFormat
}


void FAutomationControllerManager::HandleRunTestsReplyMessage(const FAutomationWorkerRunTestsReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	DeviceClusterManager.UpdateDeviceFromMessage(Context->GetSender(), Message);

	// If we should commit these results
	if ( Message.ExecutionCount == ExecutionCount )
	{
		FAutomationTestResults TestResults;

		TestResults.State = Message.State;
		TestResults.Duration = Message.Duration;

		// Mark device as back on the market
		int32 ClusterIndex;
		int32 DeviceIndex;

		verify(DeviceClusterManager.FindDevice(Message.InstanceId, ClusterIndex, DeviceIndex));
		TestResults.GameInstance = DeviceClusterManager.GetClusterGameInstanceId(ClusterIndex, DeviceIndex).ToString();
		TestResults.SetEvents(Message.Entries, Message.WarningTotal, Message.ErrorTotal);

		// Verify this device thought it was busy
		TSharedPtr<IAutomationReport> Report = DeviceClusterManager.GetTest(ClusterIndex, DeviceIndex);
		if (Report.IsValid())
		{
			Report->SetResults(ClusterIndex, CurrentTestPass, TestResults);

			const FAutomationTestResults& FinalResults = Report->GetResults(ClusterIndex, CurrentTestPass);

			ReportAutomationResult(Report, ClusterIndex, CurrentTestPass);

			// Gather all of the data relevant to this test for our json reporting.
			CollectTestResults(Report, FinalResults);
		}

		// Device is now good to go
		DeviceClusterManager.SetTest(ClusterIndex, DeviceIndex, NULL);
	}

	// Remove the running test
	RemoveTestRunning(Message.InstanceId);
}

void FAutomationControllerManager::HandleWorkerOfflineMessage( const FAutomationWorkerWorkerOffline& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	FGuid DeviceInstanceId = Message.InstanceId;
	DeviceClusterManager.Remove(DeviceInstanceId);
}

bool FAutomationControllerManager::IsDeviceGroupFlagSet( EAutomationDeviceGroupTypes::Type InDeviceGroup ) const
{
	const uint32 FlagMask = 1 << InDeviceGroup;
	return (DeviceGroupFlags & FlagMask) > 0;
}

void FAutomationControllerManager::ToggleDeviceGroupFlag( EAutomationDeviceGroupTypes::Type InDeviceGroup )
{
	const uint32 FlagMask = 1 << InDeviceGroup;
	DeviceGroupFlags = DeviceGroupFlags ^ FlagMask;
}

void FAutomationControllerManager::UpdateDeviceGroups( )
{
	DeviceClusterManager.ReGroupDevices( DeviceGroupFlags );

	// Update the reports in case the number of clusters changed
	int32 NumOfClusters = DeviceClusterManager.GetNumClusters();
	ReportManager.ClustersUpdated(NumOfClusters);
}

void FAutomationControllerManager::ResetAutomationTestTimeout(const TCHAR* Reason)
{
	//GLog->Logf(ELogVerbosity::Display, TEXT("Resetting automation test timeout: %s"), Reason);
	LastTimeUpdateTicked = FPlatformTime::Seconds();
}

#undef LOCTEXT_NAMESPACE
