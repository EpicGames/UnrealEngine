// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Containers/Queue.h"
#include "Misc/AutomationTest.h"
#include "IAutomationControllerManager.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "AutomationDeviceClusterManager.h"
#include "AutomationReportManager.h"
#include "Async/Future.h"
#include "ImageComparer.h"
#include "Interfaces/IScreenShotManager.h"
#include "Misc/EngineVersion.h"
#include "HAL/PlatformProperties.h"
#include "AutomationControllerManager.generated.h"

struct FAutomationWorkerMessageBase;

USTRUCT()
struct FAutomatedTestResult
{
	GENERATED_BODY()
public:

	TSharedPtr<IAutomationReport> Test;

	UPROPERTY()
	FString TestDisplayName;

	UPROPERTY()
	FString FullTestPath;

	UPROPERTY()
	TArray<FString> Tags;

	UPROPERTY()
	EAutomationState State;

	UPROPERTY()
	TArray<FString> DeviceInstance;

	UPROPERTY()
	float Duration;

	UPROPERTY()
	FDateTime DateTime;

	FAutomatedTestResult()
	{
		Warnings = 0;
		Errors = 0;
		State = EAutomationState::NotRun;
		Duration = 0;
		DateTime = 0;
	}

	void SetEvents(const TArray<FAutomationExecutionEntry>& InEntries, int32 InWarnings, int32 InErrors)
	{
		Entries = InEntries;
		Warnings = InWarnings;
		Errors = InErrors;
	}

	void AddEvent(EAutomationEventType EvenType, const FString& InMessage)
	{
		Entries.Add(FAutomationExecutionEntry(FAutomationEvent(EvenType, InMessage)));

		switch (EvenType)
		{
		case EAutomationEventType::Warning:
			Warnings++;
			break;
		case EAutomationEventType::Error:
			Errors++;
			break;
		default:
			break;
		}
	}

	void SetArtifacts(const TArray<FAutomationArtifact>& InArtifacts)
	{
		Artifacts = InArtifacts;
	}

	int32 GetWarningTotal() const { return Warnings; }
	int32 GetErrorTotal() const { return Errors; }

	const TArray<FAutomationExecutionEntry>& GetEntries() const { return Entries; }
	TArray<FAutomationArtifact>& GetArtifacts() { return Artifacts; }

private:

	UPROPERTY()
	TArray<FAutomationExecutionEntry> Entries;

	UPROPERTY()
	int32 Warnings;

	UPROPERTY()
	int32 Errors;

	UPROPERTY()
	TArray<FAutomationArtifact> Artifacts;
};

USTRUCT()
struct FAutomatedTestPassResults
{
	GENERATED_BODY()

public:
	FAutomatedTestPassResults()
		: ReportCreatedOn(0)
		, Succeeded(0)
		, SucceededWithWarnings(0)
		, Failed(0)
		, NotRun(0)
		, InProcess(0)
		, TotalDuration(0)
		, ComparisonExported(false)
		, IsRequired(false)
	{
	}

	UPROPERTY()
	TArray<FAutomationDeviceInfo> Devices;

	UPROPERTY()
	FDateTime ReportCreatedOn;

	UPROPERTY()
	int32 Succeeded;

	UPROPERTY()
	int32 SucceededWithWarnings;

	UPROPERTY()
	int32 Failed;

	UPROPERTY()
	int32 NotRun;

	UPROPERTY()
	int32 InProcess;

	UPROPERTY()
	float TotalDuration;

	UPROPERTY()
	bool ComparisonExported;

	UPROPERTY()
	FString ComparisonExportDirectory;

	UPROPERTY()
	TArray<FAutomatedTestResult> Tests;

	TMap<FString, uint32> TestsMapIndex;

	bool IsRequired;

	int32 GetTotalTests() const
	{
		return Succeeded + SucceededWithWarnings + Failed + NotRun + InProcess;
	}

	void AddTestResult(const IAutomationReportPtr& TestReport);

	FAutomatedTestResult& GetTestResult(const IAutomationReportPtr& TestReport);

	void ReBuildTestsMapIndex();

	bool ReflectResultStateToReport(IAutomationReportPtr& TestReport);

	void UpdateTestResultStatus(const IAutomationReportPtr& TestReport, EAutomationState State, bool bHasWarning = false);

	void ClearAllEntries()
	{
		Devices.Empty();
		Succeeded = 0;
		SucceededWithWarnings = 0;
		Failed = 0;
		NotRun = 0;
		InProcess = 0;
		TotalDuration = 0;
		IsRequired = false;
		TestsMapIndex.Empty();
		Tests.Empty();
	}
};


/**
 * Implements the AutomationController module.
 */
class FAutomationControllerManager : public IAutomationControllerManager
{
public:
	FAutomationControllerManager();

	// IAutomationController Interface
	virtual void RequestAvailableWorkers( const FGuid& InSessionId ) override;
	virtual bool IsReadyForTests() override;
	virtual void RequestTests() override;
	virtual void RunTests( const bool bIsLocalSession) override;
	virtual void StopTests() override;
	virtual void Init() override;
	virtual void RequestLoadAsset( const FString& InAssetName ) override;
	virtual void Tick() override;

	virtual void SetNumPasses(const int32 InNumPasses) override
	{
		NumTestPasses = InNumPasses;
	}

	virtual int32 GetNumPasses() override
	{
		return NumTestPasses;
	}

	virtual bool IsSendAnalytics() const override
	{
		return bSendAnalytics;
	}

	virtual void SetSendAnalytics(const bool bNewValue) override
	{
		bSendAnalytics = bNewValue;
	}

	virtual bool KeepPIEOpen() const override
	{
		return bKeepPIEOpen;
	}

	virtual void SetKeepPIEOpen(const bool bNewValue) override
	{
		bKeepPIEOpen = bNewValue;
	}

	virtual void SetFilter( TSharedPtr< AutomationFilterCollection > InFilter ) override
	{
		ReportManager.SetFilter( InFilter );
	}

	virtual TArray <TSharedPtr <IAutomationReport> >& GetFilteredReports() override
	{
		return ReportManager.GetFilteredReports();
	}

	virtual TArray <TSharedPtr <IAutomationReport> > GetEnabledReports() override
	{
		return ReportManager.GetEnabledTestReports();
	}

	virtual int32 GetNumDeviceClusters() const override
	{
		return DeviceClusterManager.GetNumClusters();
	}

	virtual int32 GetNumDevicesInCluster(const int32 ClusterIndex) const override
	{
		return DeviceClusterManager.GetNumDevicesInCluster(ClusterIndex);
	}

	virtual FString GetClusterGroupName(const int32 ClusterIndex) const override
	{
		return DeviceClusterManager.GetClusterGroupName(ClusterIndex);
	}

	virtual FString GetDeviceTypeName(const int32 ClusterIndex) const override
	{
		return DeviceClusterManager.GetClusterDeviceType(ClusterIndex);
	}

	virtual FString GetDeviceName(const int32 ClusterIndex, const int32 DeviceIndex) const override
	{
		return DeviceClusterManager.GetClusterDeviceName(ClusterIndex, DeviceIndex);
	}

	virtual FGuid GetGameInstanceId(const int32 ClusterIndex, const int32 DeviceIndex) const override
	{
		return DeviceClusterManager.GetClusterGameInstanceId(ClusterIndex, DeviceIndex);
	}

	virtual FString GetGameInstanceName(const int32 ClusterIndex, const int32 DeviceIndex) const override
	{
		return DeviceClusterManager.GetClusterGameInstance(ClusterIndex, DeviceIndex);
	}

	virtual void SetVisibleTestsEnabled(const bool bEnabled) override
	{
		ReportManager.SetVisibleTestsEnabled (bEnabled);
	}

	virtual int32 GetEnabledTestsNum() const override
	{
		return ReportManager.GetEnabledTestsNum();
	}

	virtual void GetEnabledTestNames(TArray<FString>& OutEnabledTestNames) const override
	{
		ReportManager.GetEnabledTestNames(OutEnabledTestNames);
	}

	virtual void GetFilteredTestNames(TArray<FString>& OutFilteredTestNames) const override
	{
		ReportManager.GetFilteredTestNames(OutFilteredTestNames);
	}

	virtual void SetEnabledTests(const TArray<FString>& EnabledTests) override
	{
		ReportManager.SetEnabledTests(EnabledTests);
	}

	virtual EAutomationControllerModuleState::Type GetTestState( ) const override
	{
		return AutomationTestState;
	}

	virtual void SetDeveloperDirectoryIncluded(const bool bInDeveloperDirectoryIncluded) override
	{
		bDeveloperDirectoryIncluded = bInDeveloperDirectoryIncluded;
	}

	virtual bool IsDeveloperDirectoryIncluded(void) const override
	{
		return bDeveloperDirectoryIncluded;
	}

	virtual void SetRequestedTestFlags(const EAutomationTestFlags InRequestedTestFlags) override
	{
		RequestedTestFlags = InRequestedTestFlags;
		RequestTests();
	}

	virtual const bool CheckTestResultsAvailable() const override
	{
		return 	bTestResultsAvailable;
	}

	virtual const bool ReportsHaveErrors() const override
	{
		return bHasErrors;
	}

	virtual const bool ReportsHaveWarnings() const override
	{
		return bHasWarning;
	}

	virtual const bool ReportsHaveLogs() const override
	{
		return bHasLogs;
	}

	virtual void ClearAutomationReports() override
	{
		ReportManager.Empty();
	}

	virtual const bool ExportReport(uint32 FileExportTypeMask) override;
	virtual bool IsTestRunnable( IAutomationReportPtr InReport ) const override;
	virtual void RemoveCallbacks() override;
	virtual void Shutdown() override;
	virtual void Startup() override;

	virtual FOnAutomationControllerManagerShutdown& OnShutdown( ) override
	{
		return ShutdownDelegate;
	}

	virtual FOnAutomationControllerManagerTestsAvailable& OnTestsAvailable( ) override
	{
		return TestsAvailableDelegate;
	}

	virtual FOnAutomationControllerTestsRefreshed& OnTestsRefreshed( ) override
	{
		return TestsRefreshedDelegate;
	}

	virtual FOnAutomationControllerTestsComplete& OnTestsComplete() override
	{
		return TestsCompleteDelegate;
	}

	virtual FOnAutomationControllerReset& OnControllerReset() override
	{
		return ControllerResetDelegate;
	}


	virtual bool IsDeviceGroupFlagSet( EAutomationDeviceGroupTypes::Type InDeviceGroup ) const override;
	virtual void ToggleDeviceGroupFlag( EAutomationDeviceGroupTypes::Type InDeviceGroup ) override;
	virtual void UpdateDeviceGroups() override;

	virtual FString GetReportOutputPath() const override;

	virtual void ResetAutomationTestTimeout(const TCHAR* Reason) override;
	
protected:

	/**
	 * Adds a ping result from a running test.
	 *
	 * @param ResponderInstanceId The worker instance identifier that responded to a ping.
	 */
	void AddPingResult(const FGuid& ResponderInstanceId);

	/**
	* Spew all of our results of the test out to the log.
	*/
	void ReportTestResults();

	/**
	 * Generates a serialized json string that contains all of our test report data
	 */
	bool GenerateJsonTestPassSummary(FAutomatedTestPassResults& SerializedPassResults, FString& OutTestReportJson);

	/**
	 * Create a json file that contains all of our test report data at /saved/automation/logs/AutomationReport-{CL}-{DateTime}.json
	 */
	bool SaveJsonTestPassSummary(const FString& TestReportJson);

	/**
	 * Braodcasts a serialized json string that contains all of our test report data
	 */
	void BroadcastJsonTestPassSummary(const FString& TestReportJson);

	/**
	 * Generates a full html report of the testing, which may include links to images.  All of it will be bundled under a folder.
	 */
	bool GenerateTestPassHtmlIndex();

	/**
	 * Load test results from previous json test pass summary file and reflect results on reports
	 */
	bool LoadJsonTestPassSummary(FString& ReportFilePath, TArray<IAutomationReportPtr> TestReports);

	/**
	* Gather all info, warning, and error lines generated over the course of a test.
	*
	* @param TestName The test that was just run.
	* @param TestResult All of the messages of note generated during the test case.
	*/
	void CollectTestResults(TSharedPtr<IAutomationReport> Report, const FAutomationTestResults& Results);

	/**
	 * Checks the child result.
	 *
	 * @param InReport The child result to check.
	 */
	void CheckChildResult( TSharedPtr< IAutomationReport > InReport );

	FString SlugString(const FString& DisplayString) const;

	FString CopyArtifact(const FString& DestFolder, const FString& SourceFile) const;

	/**
	 * Execute the next task thats available.
	 *
	 * @param ClusterIndex The Cluster index of the device type we intend to use.
	 * @param bAllTestsCompleted Whether all tests have been completed.
	 */
	void ExecuteNextTask( int32 ClusterIndex, OUT bool& bAllTestsCompleted );

	/* Report an image comparison result */
	void ReportImageComparisonResult(const FAutomationWorkerImageComparisonResults& Result);

	/** Process the comparison queue to see if there are comparisons we need to respond to the test with. */
	void ProcessComparisonQueue();

	/** Distributes any tests that are pending and deal with tests finishing. */
	void ProcessAvailableTasks();

	/** Processes the results after tests are complete. */
	void ProcessResults();

	/**
	 * Removes the test info.
	 *
	 * @param OwnerInstanceId Instance identifier of the test to remove.
	 */
	void RemoveTestRunning(const FGuid& OwnerInstanceId);

	/** Changes the controller state. */
	void SetControllerStatus( EAutomationControllerModuleState::Type AutomationTestState );

	/** Stores the tests that are valid for a particular device classification. */
	void SetTestNames(const FGuid& AutomationWorkerInstanceId, TArray<FAutomationTestInfo>& TestInfo);

	/** Updates the tests to ensure they are all still running. */
	void UpdateTests();

	/** Sends stop session message to worker instances. */
	void StopStartedTestSessions();

	/**
	 * Send a message in an unified way with passing correct Instance Id into the message we are going to send.
	 *
	 * @param Message The message to be sent.
	 * @param TypeInfo The type information about the message to be sent.
	 * @param ControllerAddress The message address of the receiver.
	 */
	void SendMessage(FAutomationWorkerMessageBase* Message, UScriptStruct* TypeInfo, const FMessageAddress& ControllerAddress);

private:

	/** Handles FAutomationWorkerFindWorkersResponse messages. */
	void HandleFindWorkersResponseMessage( const FAutomationWorkerFindWorkersResponse& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerPong messages. */
	void HandlePongMessage( const FAutomationWorkerPong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerScreenImage messages. */
	void HandleReceivedScreenShot( const FAutomationWorkerScreenImage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerScreenshotComparisonResult messages. */
	void HandleReceivedComparisonResult( const FAutomationWorkerImageComparisonResults& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerTestDataRequest messages. */
	void HandleTestDataRequest(const FAutomationWorkerTestDataRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerPerformanceDataRequest messages. */
	void HandlePerformanceDataRequest(const FAutomationWorkerPerformanceDataRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerRequestNextNetworkCommand messages. */
	void HandleRequestNextNetworkCommandMessage( const FAutomationWorkerRequestNextNetworkCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerRequestTestsReplyComplete messages. */
	void HandleRequestTestsReplyCompleteMessage(const FAutomationWorkerRequestTestsReplyComplete& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FAutomationWorkerRunTestsReply messages. */
	void HandleRunTestsReplyMessage( const FAutomationWorkerRunTestsReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerWorkerOffline messages. */
	void HandleWorkerOfflineMessage( const FAutomationWorkerWorkerOffline& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FAutomationWorkerTelemetryData messages. */
	void HandleReceivedTelemetryData(const FAutomationWorkerTelemetryData& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Writes out this automation result to the log */
	void ReportAutomationResult(const TSharedPtr<IAutomationReport> InReport, int32 ClusterIndex, int32 PassIndex);
private:

	/** Session this controller is currently communicating with */
	FGuid ActiveSessionId;

	/** The automation test state */
	EAutomationControllerModuleState::Type AutomationTestState;

	/** Which grouping flags are enabled */
	uint32 DeviceGroupFlags = 0;

	/** Whether to include developer content in the automation tests */
	bool bDeveloperDirectoryIncluded = false;

	/** Some tests have errors */
	bool bHasErrors = false;

	/** Some tests have warnings */
	bool bHasWarning = false;

	/** Some tests have logs */
	bool bHasLogs = false;

	/** Is this a local session */
	bool bIsLocalSession;

	/** Are tests results available */
	bool bTestResultsAvailable = false;

	/** Which sets of tests to consider */
	EAutomationTestFlags RequestedTestFlags = EAutomationTestFlags::None;

	/** Timer to keep track of the last time tests were updated */
	double CheckTestTimer;

	/** The time to wait between test updates in seconds */
	float CheckTestIntervalSeconds = 1.0f;
	
	/** The time to wait before considering a game instance as lost in seconds */
	float GameInstanceLostTimerSeconds = 300.0f;

	/** Whether tick is still executing tests for different clusters */
	uint32 ClusterDistributionMask;

	/** Available worker GUIDs */
	FAutomationDeviceClusterManager DeviceClusterManager;

	/** The iteration number of executing the tests.  Ensures restarting the tests won't allow stale results to try and commit */
	uint32 ExecutionCount = 0;

	/** Last time the update tests function was ticked */
	double LastTimeUpdateTicked = 0;

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Counter for number of workers we have received responses from for Refreshing the Test List */
	uint32 RefreshTestResponses = 0;

	/** Available stats/status for all tests. */
	FAutomationReportManager ReportManager;

	/** A data holder to keep track of how long tests have been running. */
	struct FTestRunningInfo
	{
		FTestRunningInfo( FMessageAddress InMessageAddress, FGuid InInstanceId):
			OwnerMessageAddress( InMessageAddress),
			OwnerInstanceId( InInstanceId ),
			LastPingTime( 0.f )
		{
		}
		/** The test runners message address */
		FMessageAddress OwnerMessageAddress;
		/** The test runner's instance ID */
		FGuid OwnerInstanceId;
		/** The time since we had a ping from the instance*/
		double LastPingTime;
	};

	/** A array of running tests. */
	TArray< FTestRunningInfo > TestRunningArray;

	/** Set of worker instance identifiers for started sessions. */
	TSet< FGuid > StartedTestSessionWorkerInstanceIdSet;

	/** The number of test passes to perform. */
	int32 NumTestPasses = 0;

	/** The current test pass we are on. */
	int32 CurrentTestPass = 0;

	/** If we should send result to analytics */
	bool bSendAnalytics = false;

	/** If we should send keep the PIE open when test pass end */
	bool bKeepPIEOpen = false;

	/** If we should sort tests by failure type in json/html report */
	bool bSortTestsByFailure = false;

	/** If we should prune log events from test report on success */
	bool bPruneLogsOnSuccess = true;

	/** If the test requires any sort of reporting to be handled (delegate or writing to file) */
	bool bRequiresReportGeneration = false;

	/** The list of results generated by our test pass. */
	FAutomatedTestPassResults JsonTestPassResults;

	/** The screenshot manager. */
	IScreenShotManagerPtr ScreenshotManager;

	/**
	 * Holds the information required to perform a screen comparison asynchronously.
	 */
	struct FComparisonEntry
	{
		FMessageAddress Sender;
		FGuid InstanceId;
		TFuture<FImageComparisonResult> PendingComparison;
	};

	/** Pending image comparisons */
	TQueue<TSharedPtr<FComparisonEntry>> ComparisonQueue;

	/** The report folder override path that may have been provided over the commandline, -ReportOutputPath="" */
	FString ReportExportPath;
	FString ReportURLPath;

	FString DeveloperReportUrl;

	bool bResumeRunTest;

#if WITH_EDITOR && !UE_BUILD_SHIPPING && WITH_AUTOMATION_TESTS
	TSharedPtr<class FWaitForInteractiveFrameRate> InteractiveFrameRateCheck;
#endif

private:

	/** Holds a delegate that is invoked when the controller shuts down. */
	FOnAutomationControllerManagerShutdown ShutdownDelegate;

	/** Holds a delegate that is invoked when the controller has tests available. */
	FOnAutomationControllerManagerTestsAvailable TestsAvailableDelegate;

	/** Holds a delegate that is invoked when the controller's tests are being refreshed. */
	FOnAutomationControllerTestsRefreshed TestsRefreshedDelegate;

	/** Holds a delegate that is invoked when the controller's reset. */
	FOnAutomationControllerReset ControllerResetDelegate;	

	/** Holds a delegate that is invoked when the tests have completed. */
	FOnAutomationControllerTestsComplete TestsCompleteDelegate;
};
