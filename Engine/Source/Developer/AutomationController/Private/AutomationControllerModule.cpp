// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AutomationControllerModule.cpp: Implements the FAutomationControllerModule class.
=============================================================================*/

#include "AutomationControllerModule.h"
#include "AutomationControllerManager.h"

#if WITH_EDITOR
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#endif


#define LOCTEXT_NAMESPACE "AutomationTesting"

IMPLEMENT_MODULE(FAutomationControllerModule, AutomationController);

IAutomationControllerManagerRef FAutomationControllerModule::GetAutomationController( )
{
	if (!AutomationControllerSingleton.IsValid())
	{
		AutomationControllerSingleton = MakeShareable(new FAutomationControllerManager());
	}

	return AutomationControllerSingleton.ToSharedRef();
}

void FAutomationControllerModule::Init()
{
	GetAutomationController()->Init();
}

void FAutomationControllerModule::Tick()
{
	GetAutomationController()->Tick();
}

void FAutomationControllerModule::ShutdownModule()
{
	GetAutomationController()->Shutdown();
	AutomationControllerSingleton.Reset();

#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("AutomationTestingLog");
	}
#endif
}

void FAutomationControllerModule::StartupModule()
{
	GetAutomationController()->Startup();

#if WITH_EDITOR
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = true;
	InitOptions.bScrollToBottom = true;
	MessageLogModule.RegisterLogListing("AutomationTestingLog", LOCTEXT("AutomationTesting", "Automation Testing Log"), InitOptions);
#endif
}

bool FAutomationControllerModule::SupportsDynamicReloading()
{
	return true;
}

/* Static initialization
 *****************************************************************************/

IAutomationControllerManagerPtr FAutomationControllerModule::AutomationControllerSingleton = NULL;

#undef LOCTEXT_NAMESPACE
