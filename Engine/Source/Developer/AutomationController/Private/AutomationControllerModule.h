// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAutomationControllerModule.h"

/**
 * Implements the AutomationController module.
 */
class FAutomationControllerModule
	: public IAutomationControllerModule
{
public:

	// IAutomationControllerModule interface

	virtual IAutomationControllerManagerRef GetAutomationController( ) override;
	virtual void Init() override;
	virtual void Tick() override;

public:

	// IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;

private:

	/** Holds the automation controller singleton. */
	static IAutomationControllerManagerPtr AutomationControllerSingleton;
};
