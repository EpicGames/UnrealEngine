// Copyright Epic Games, Inc. All Rights Reserved.


#include "AutomationControllerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationControllerSettings)

DEFINE_LOG_CATEGORY_STATIC(LogAutomationControllerSettings, Log, All)

UAutomationControllerSettings::UAutomationControllerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bSuppressLogErrors(false)
	, bSuppressLogWarnings(false)
	, bElevateLogWarningsToErrors(true)
	, bKeepPIEOpen(false)
	, bAutoExpandSingleItemSubgroups(true)
	, bSortTestsByFailure(false)
	, bPruneLogsOnSuccess(true)
	, bTreatLogWarningsAsTestErrors(true)
	, GameInstanceLostTimerSeconds(300.0f)
	, bResetTelemetryStorageOnNewSession(false)
{
}

void UAutomationControllerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (!bTreatLogWarningsAsTestErrors)
	{
		UE_LOGF(LogAutomationControllerSettings, Warning, "UAutomationControllerSettings::bTreatLogWarningsAsTestErrors is deprecated. Use bElevateLogWarningsToErrors instead.");
		bElevateLogWarningsToErrors = false;
	}
}
