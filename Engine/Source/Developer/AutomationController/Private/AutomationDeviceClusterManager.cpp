// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDeviceClusterManager.h"
#include "IAutomationControllerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationDeviceClusterManager)

void FAutomationDeviceClusterManager::Reset()
{
	Clusters.Empty();
}

void FAutomationDeviceClusterManager::AddDeviceFromMessage(const FMessageAddress& MessageAddress, const FAutomationWorkerFindWorkersResponse& Message, const uint32 GroupFlags)
{
	check(Message.InstanceId.IsValid());

	int32 TestClusterIndex = INDEX_NONE;
	int32 TestDeviceIndex = INDEX_NONE;
	//if we don't already know about this device
	if (!FindDevice(Message.InstanceId, TestClusterIndex, TestDeviceIndex))
	{
		FDeviceState NewDevice(MessageAddress, Message);
		FString GroupName = GetGroupNameForDevice(NewDevice, GroupFlags);
		//ensure the proper cluster exists
		int32 ClusterIndex = 0;
		for (; ClusterIndex < Clusters.Num(); ++ClusterIndex)
		{
			if (Clusters[ClusterIndex].ClusterName == GroupName)
			{
				//found the cluster, now just append the device
				Clusters[ClusterIndex].Devices.Add(NewDevice);
				break;
			}
		}
		// if we didn't find the device type yet, add a new cluster and add this device
		if (ClusterIndex == Clusters.Num())
		{
			FDeviceCluster NewCluster;
			NewCluster.ClusterName = GroupName;
			NewCluster.DeviceTypeName = Message.Platform;
			NewCluster.Devices.Add(NewDevice);
			Clusters.Add(NewCluster);
		}
	}
	else
	{
		UpdateDeviceFromMessage(MessageAddress, Message);
	}
}

void FAutomationDeviceClusterManager::UpdateDeviceFromMessage(const FMessageAddress& MessageAddress, const FAutomationWorkerMessageBase& Message)
{
	check(Message.InstanceId.IsValid());

	int32 TestClusterIndex = INDEX_NONE;
	int32 TestDeviceIndex = INDEX_NONE;

	if (FindDevice(Message.InstanceId, TestClusterIndex, TestDeviceIndex))
	{
		// If we already know about this device
		Clusters[TestClusterIndex].Devices[TestDeviceIndex].DeviceMessageAddress = MessageAddress;
	}
}

void FAutomationDeviceClusterManager::Remove(const FGuid& DeviceInstanceId)
{
	for (int32 ClusterIndex = 0; ClusterIndex < Clusters.Num(); ++ClusterIndex)
	{
		for (int32 DeviceIndex = Clusters[ClusterIndex].Devices.Num()-1; DeviceIndex >= 0; --DeviceIndex)
		{
			if (DeviceInstanceId == Clusters[ClusterIndex].Devices[DeviceIndex].Info.Instance)
			{
				Clusters[ClusterIndex].Devices.RemoveAt(DeviceIndex);
			}
		}
	}
}


FString FAutomationDeviceClusterManager::GetGroupNameForDevice(const FDeviceState& DeviceState, const uint32 DeviceGroupFlags)
{
	FString OutGroupName;

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::MachineName)) > 0 )
	{
		OutGroupName += DeviceState.Info.DeviceName + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::Platform)) > 0 )
	{
		OutGroupName += DeviceState.Info.Platform + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::OSVersion)) > 0 )
	{
		OutGroupName += DeviceState.Info.OSVersion + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::Model)) > 0 )
	{
		OutGroupName += DeviceState.Info.Model + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::GPU)) > 0 )
	{
		OutGroupName += DeviceState.Info.GPU + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::CPUModel)) > 0 )
	{
		OutGroupName += DeviceState.Info.CPUModel + TEXT("-");
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::RamInGB)) > 0 )
	{
		OutGroupName += FString::Printf(TEXT("%uGB Ram-"),DeviceState.Info.RAMInGB);
	}

	if( (DeviceGroupFlags & (1 << EAutomationDeviceGroupTypes::RenderMode)) > 0 )
	{
		OutGroupName += DeviceState.Info.RenderMode + TEXT("-");
	}

	if( OutGroupName.Len() > 0 )
	{
		//Get rid of the trailing '-'
		OutGroupName.LeftChopInline(1, EAllowShrinking::No);
	}

	return OutGroupName;
}


void FAutomationDeviceClusterManager::ReGroupDevices( const uint32 GroupFlags )
{
	//Get all the devices
	TArray< FDeviceState > AllDevices;
	for(int32 i=0; i<GetNumClusters(); ++i)
	{
		AllDevices += Clusters[i].Devices;
	}

	//Clear out the clusters
	Reset();

	//Generate new group names based off the active flags and readd the devices
	for(int32 i=0; i<AllDevices.Num(); ++i)
	{
		const FDeviceState* DeviceIt = &AllDevices[i];
		FString GroupName = GetGroupNameForDevice(*DeviceIt, GroupFlags);
		//ensure the proper cluster exists
		int32 ClusterIndex = 0;
		for (; ClusterIndex < Clusters.Num(); ++ClusterIndex)
		{
			if (Clusters[ClusterIndex].ClusterName == GroupName)
			{
				//found the cluster, now just append the device
				Clusters[ClusterIndex].Devices.Add(*DeviceIt);
				break;
			}
		}
		// if we didn't find the device type yet, add a new cluster and add this device
		if (ClusterIndex == Clusters.Num())
		{
			FDeviceCluster NewCluster;
			NewCluster.ClusterName = GroupName;
			NewCluster.DeviceTypeName = DeviceIt->Info.Platform;
			NewCluster.Devices.Add(*DeviceIt);
			Clusters.Add(NewCluster);
		}
	}
}


int32 FAutomationDeviceClusterManager::GetNumClusters() const
{
	return Clusters.Num();
}


int32 FAutomationDeviceClusterManager::GetTotalNumDevices() const
{
	int Total = 0;
	for (int32 ClusterIndex = 0; ClusterIndex < Clusters.Num(); ++ClusterIndex)
	{
		Total += Clusters[ClusterIndex].Devices.Num();
	}
	return Total;
}


int32 FAutomationDeviceClusterManager::GetNumDevicesInCluster(const int32 ClusterIndex) const
{
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	return Clusters[ClusterIndex].Devices.Num();
}


int32 FAutomationDeviceClusterManager::GetNumActiveDevicesInCluster(const int32 ClusterIndex) const
{
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	int32 ActiveDevices = 0;
	for ( int32 Index = 0; Index < Clusters[ ClusterIndex ].Devices.Num(); Index++ )
	{
		if ( Clusters[ ClusterIndex ].Devices[ Index ].IsDeviceAvailable )
		{
			ActiveDevices++;
		}
	}
	return ActiveDevices;
}


FString FAutomationDeviceClusterManager::GetClusterGroupName(const int32 ClusterIndex) const
{
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	return Clusters[ClusterIndex].ClusterName;
}


FString FAutomationDeviceClusterManager::GetClusterDeviceType(const int32 ClusterIndex) const
{
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	return Clusters[ClusterIndex].DeviceTypeName;
}

FString FAutomationDeviceClusterManager::GetClusterDeviceName(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	return GetDeviceInfo(ClusterIndex, DeviceIndex).DeviceName;
}

FString FAutomationDeviceClusterManager::GetClusterGameInstance(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	return GetDeviceInfo(ClusterIndex, DeviceIndex).InstanceName;
}

FGuid FAutomationDeviceClusterManager::GetClusterGameInstanceId(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	return GetDeviceInfo(ClusterIndex, DeviceIndex).Instance;
}


const FAutomationDeviceInfo& FAutomationDeviceClusterManager::GetDeviceInfo(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	return Clusters[ClusterIndex].Devices[DeviceIndex].Info;
}

bool FAutomationDeviceClusterManager::FindDevice(const FGuid& InstanceId, int32& OutClusterIndex, int32& OutDeviceIndex)
{
	OutClusterIndex = INDEX_NONE;
	OutDeviceIndex = INDEX_NONE;
	for (int32 ClusterIndex = 0; ClusterIndex < Clusters.Num(); ++ClusterIndex)
	{
		for (int32 DeviceIndex = 0; DeviceIndex < Clusters[ClusterIndex].Devices.Num(); ++DeviceIndex)
		{
			//if network addresses match
			if (InstanceId == Clusters[ClusterIndex].Devices[DeviceIndex].Info.Instance)
			{
				OutClusterIndex = ClusterIndex;
				OutDeviceIndex = DeviceIndex;
				return true;
			}
		}
	}
	return false;
}

FMessageAddress FAutomationDeviceClusterManager::GetDeviceMessageAddress(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	//verify cluster/device index
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	return Clusters[ClusterIndex].Devices[DeviceIndex].DeviceMessageAddress;
}


TArray<FMessageAddress> FAutomationDeviceClusterManager::GetDevicesReservedForTest(const int32 ClusterIndex, TSharedPtr <IAutomationReport> Report)
{
	TArray<FMessageAddress> DeviceAddresses;

	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	for (int32 DeviceIndex = 0; DeviceIndex < Clusters[ClusterIndex].Devices.Num(); ++DeviceIndex)
	{
		if (Clusters[ClusterIndex].Devices[DeviceIndex].Report == Report)
		{
			DeviceAddresses.Add(Clusters[ClusterIndex].Devices[DeviceIndex].DeviceMessageAddress);
		}
	}
	return DeviceAddresses;
}


TSharedPtr <IAutomationReport> FAutomationDeviceClusterManager::GetTest(const int32 ClusterIndex, const int32 DeviceIndex) const
{
	//verify cluster/device index
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	return Clusters[ClusterIndex].Devices[DeviceIndex].Report;
}


void FAutomationDeviceClusterManager::SetTest(const int32 ClusterIndex, const int32 DeviceIndex, TSharedPtr <IAutomationReport> NewReport)
{
	//verify cluster/device index
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	Clusters[ClusterIndex].Devices[DeviceIndex].Report = NewReport;
}


void FAutomationDeviceClusterManager::ResetAllDevicesRunningTest( const int32 ClusterIndex, IAutomationReportPtr InTest )
{	
	TArray<FMessageAddress> DeviceAddresses;

	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	for (int32 DeviceIndex = 0; DeviceIndex < Clusters[ClusterIndex].Devices.Num(); ++DeviceIndex)
	{
		if( Clusters[ClusterIndex].Devices[DeviceIndex].Report == InTest )
		{
			Clusters[ClusterIndex].Devices[DeviceIndex].Report = NULL;
		}
	}
}


void FAutomationDeviceClusterManager::DisableDevice( const int32 ClusterIndex, const int32 DeviceIndex )
{
	//verify cluster/device index
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	Clusters[ClusterIndex].Devices[DeviceIndex].IsDeviceAvailable = false;
}


bool FAutomationDeviceClusterManager::DeviceEnabled( const int32 ClusterIndex, const int32 DeviceIndex )
{
	//verify cluster/device index
	check((ClusterIndex >= 0) && (ClusterIndex < Clusters.Num()));
	check((DeviceIndex >= 0) && (DeviceIndex < Clusters[ClusterIndex].Devices.Num()));
	return Clusters[ClusterIndex].Devices[DeviceIndex].IsDeviceAvailable;
}


bool FAutomationDeviceClusterManager::HasActiveDevice()
{
	bool IsDeviceAvailable = false;
	for (int32 ClusterIndex = 0; ClusterIndex < Clusters.Num(); ++ClusterIndex)
	{
		for (int32 DeviceIndex = 0; DeviceIndex < Clusters[ClusterIndex].Devices.Num(); ++DeviceIndex)
		{
			//if network addresses match
			if ( Clusters[ClusterIndex].Devices[DeviceIndex].IsDeviceAvailable )
			{
				IsDeviceAvailable = true;
				break;
			}
		}
	}
	return IsDeviceAvailable;
}

