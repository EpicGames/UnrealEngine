// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "IAutomationReport.h"
#include "AutomationWorkerMessages.h"
#include "IMessageContext.h"
#include "AutomationDeviceClusterManager.generated.h"

/**
* Hold information about the Device
*/
USTRUCT()
struct FAutomationDeviceInfo
{
	GENERATED_BODY()
public:
	/** The name of device */
	UPROPERTY()
	FString DeviceName;

	/** The instance ID */
	UPROPERTY()
	FGuid Instance;

	/** The instance Name */
	UPROPERTY()
	FString InstanceName;

	/** The name of the platform */
	UPROPERTY()
	FString Platform;

	/** The name of the operating system version */
	UPROPERTY()
	FString OSVersion;

	/** The name of the device model */
	UPROPERTY()
	FString Model;

	/** The name of the GPU */
	UPROPERTY()
	FString GPU;

	/** The name of the CPU model */
	UPROPERTY()
	FString CPUModel;

	/** The amount of RAM this device has in gigabytes */
	UPROPERTY()
	uint32 RAMInGB = 0;

	/** The name of the current render mode */
	UPROPERTY()
	FString RenderMode;

	/** The name of the current RHI */
	UPROPERTY()
	FString RHI;

	/** The path to the application instance log */
	UPROPERTY()
	FString AppInstanceLog;
};

/**
 * Managers groups of devices for job distribution
 */
class FAutomationDeviceClusterManager
{
public:
	/** Clear out all clusters for a new session */
	void Reset();

	/**
	 * Add a new device from an AutomationWorkerFindWorkersResponse message, creating new clusters as needed.
	 *
	 * @param MessageAddress The network address of an available worker.
	 * @param Message The message that contains the device info.
	 * @param GroupFlags Used find out which group to put the new device into.
	 */
	void AddDeviceFromMessage(const FMessageAddress& MessageAddress, const FAutomationWorkerFindWorkersResponse& Message, const uint32 GroupFlags);

	/**
	 * Update existing device's network address.
	 *
	 * @param MessageAddress The network address of an available worker.
	 * @param Message The message that contains the device info.
	 */
	void UpdateDeviceFromMessage(const FMessageAddress& MessageAddress, const FAutomationWorkerMessageBase& Message);

	/** Remove a device (went offline, etc). */
	void Remove(const FGuid& DeviceInstanceId);

	/** Returns number of unique device types. */
	int32 GetNumClusters() const;

	/** Returns the sum of the device counts per cluster. */
	int32 GetTotalNumDevices() const;

	/** Returns the number of devices in of a particular device type. */
	int32 GetNumDevicesInCluster (const int32 ClusterIndex) const;

	/** Returns the number of active devices in of a particular device type. */
	int32 GetNumActiveDevicesInCluster(const int32 ClusterIndex) const;

	/** Returns the name of the cluster group. */
	FString GetClusterGroupName(const int32 ClusterIndex) const;

	/** Returns the name of the devices within this cluster. */
	FString GetClusterDeviceType (const int32 ClusterIndex) const;

	/** Returns the name of a device within a cluster. */
	FString GetClusterDeviceName(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/** Returns the instance of the game within a cluster. */
	FString GetClusterGameInstance(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/** Returns the instance identifier of the game within a cluster. */
	FGuid GetClusterGameInstanceId(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/** Returns the name of a device within a cluster. */
	const FAutomationDeviceInfo& GetDeviceInfo(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/**
	 * Finds the cluster/device index for a particular GUID.
	 *
	 * @param InstanceId The device instance identifier.
	 * @param OutClusterIndex The index of the platform this device was in.
	 * @param OutDeviceIndex The index of the device within OutClusterIndex.
	 * @return true if the device was found within a cluster.
	 */
	bool FindDevice(const FGuid& InstanceId, int32& OutClusterIndex, int32& OutDeviceIndex);

	/** 
	 * Returns the message address of the device specified.
	 *
	 * @param ClusterIndex Index of the device cluster.
	 * @param DeviceIndex Index of the device within the cluster.
	 *
	 * @return The message address of this device
	 */
	FMessageAddress GetDeviceMessageAddress(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/**
	 * Returns the devices that have been reserved for a particular test.
	 *
	 * @param ClusterIndex The device type we are testing.
	 * @param Report The automation test that we are trying to run (when we have enough participants).
	 * @return The array of devices that are currently signed up for running this test.
	 */
	TArray<FMessageAddress> GetDevicesReservedForTest(const int32 ClusterIndex, TSharedPtr <IAutomationReport> Report);

	/**
	 * Returns the current test this device should be running.
	 *
	 * @param ClusterIndex Index of the device cluster.
	 * @param DeviceIndex Index of the device within the cluster.
	 * @return nullptr if this device is available or the test structure if it is in-process.
	 */
	TSharedPtr <IAutomationReport> GetTest(const int32 ClusterIndex, const int32 DeviceIndex) const;

	/**
	 * Sets the current test being run by the particular device.
	 *
	 * @param ClusterIndex The Cluster which holds the device we intend to run the test on.
	 * @param DeviceIndex The devices index in the cluster where we intend to run the test on.
	 * @param NewReport The test we are going to run.
	 */
	void SetTest(const int32 ClusterIndex, const int32 DeviceIndex, TSharedPtr <IAutomationReport> NewReport);

	/**
	 * Resets all cases on devices which are running the specified test to NULL.
	 *
	 * @param ClusterIndex The Cluster where we want all devices running the test to cease.
	 * @param InTest The test we are stopping.
	 */
	void ResetAllDevicesRunningTest( const int32 ClusterIndex, IAutomationReportPtr InTest );

	/**
	 * Disable a device as it is no longer available - keep around to get the results.
	 *
	 * @param ClusterIndex The index of the platform this device was in.
	 * @param DeviceIndex The index of the device within OutClusterIndex.
	 */
	void DisableDevice( const int32 ClusterIndex, const int32 DeviceIndex );

	/**
	 * Check if a device is enabled.
	 *
	 * @param ClusterIndex The index of the platform this device was in.
	 * @param DeviceIndex The index of the device within OutClusterIndex.
	 */
	bool DeviceEnabled( const int32 ClusterIndex, const int32 DeviceIndex );

	/**
	 * Check if there are any active game instances left.
	 *
	 * @return True if there are some game instances.
	 */
	bool HasActiveDevice();

	/** Regroups the device clusters based off the current group flags. */
	void ReGroupDevices( const uint32 GroupFlags );

private:

	/** per actual device, the network address and current test being run */
	class FDeviceState
	{
	public:
		FDeviceState(FMessageAddress NewMessageAddress, const FAutomationWorkerFindWorkersResponse& Message )
		{
			DeviceMessageAddress = NewMessageAddress;
			Info.DeviceName = Message.DeviceName;
			Info.Platform = Message.Platform;
			Info.OSVersion = Message.OSVersionName;
			Info.Model = Message.ModelName;
			Info.GPU = Message.GPUName;
			Info.CPUModel = Message.CPUModelName;
			Info.RAMInGB = Message.RAMInGB;
			Info.RenderMode = Message.RenderModeName;
			Info.RHI = Message.RHIName;
			Info.Instance = Message.InstanceId;
			Info.InstanceName = Message.InstanceName;
			Report.Reset();
			IsDeviceAvailable = true;
		}

		/** Network address for device */
		FMessageAddress DeviceMessageAddress;

		/** The Device full info */
		FAutomationDeviceInfo Info;

		/** NULL if this device is available to do work*/
		TSharedPtr <IAutomationReport> Report;

		/** Hold if the game instance is available */
		bool IsDeviceAvailable;
	};

	/** for each cluster, the name and array of known devices */
	class FDeviceCluster
	{
	public:
		/** Name of the Cluster */
		FString ClusterName;

		/** Name of the platform */
		FString DeviceTypeName;

		/** Guids/state for each device of this type */
		TArray <FDeviceState> Devices;
	};

	/**
	 * Generates a group name based off the device info and current group flags
	 */
	FString GetGroupNameForDevice(const FDeviceState& DeviceState, const uint32 DeviceGroupFlags);

	/** Array of all clusters */
	TArray <FDeviceCluster> Clusters;
};
