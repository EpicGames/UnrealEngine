// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationReport.h"
#include "AutomationTestExcludelist.h"
#include "Misc/FilterCollection.h"

FAutomationReport::FAutomationReport(FAutomationTestInfo& InTestInfo, bool InIsParent)
	: bEnabled( false )
	, bIsParent(InIsParent)
	, bNodeExpandInUI(false)
	, bSelfPassesFilter(false)
	, SupportFlags(0)
	, TestInfo( InTestInfo )
{
	// Enable smoke tests
	if ( TestInfo.GetTestFlags() == EAutomationTestFlags::SmokeFilter )
	{
		bEnabled = true;
	}

	// Get exclude test info from config for every supported platform. We want to avoid querying when TestInfo is empty
	// (e.g. this is the empty root node in FAutomationControllerManager and we don't actually need to load config yet).
	// This saves ~340ms when opening the editor.
	const FString& TestPath = TestInfo.GetFullTestPath();
	if (!TestPath.IsEmpty())
	{
		if (auto Entry = UAutomationTestExcludelist::Get()->GetExcludeTestEntry(TestPath))
		{
			ExcludeTestInfo = *Entry;
			ExcludeTestInfo.SetPropagation(TestPath);
			bNeedToSkip = true;
		}
	}
}

void FAutomationReport::Empty()
{
	//release references to all child tests
	ChildReports.Empty();
	ChildReportNameHashes.Empty();
	FilteredChildReports.Empty();
}

FString FAutomationReport::GetTestParameter() const
{
	return TestInfo.GetTestParameter();
}

FString FAutomationReport::GetTags() const
{
	return FAutomationTestFramework::Get().GetTagsForAutomationTest(this->GetFullTestPath());
}

TArray<FString> FAutomationReport::GetTagsArray() const
{
	return FAutomationTestFramework::Get().GetTagsForAutomationTestAsArray(this->GetFullTestPath());
}

FString FAutomationReport::GetAssetPath() const
{
	return TestInfo.GetAssetPath();
}

FString FAutomationReport::GetOpenCommand() const
{
	return TestInfo.GetOpenCommand();
}

FString FAutomationReport::GetCommand() const
{
	return TestInfo.GetTestName();
}

const FString& FAutomationReport::GetDisplayName() const
{
	return TestInfo.GetDisplayName();
}

const FString& FAutomationReport::GetFullTestPath() const
{
	return TestInfo.GetFullTestPath();
}

FString FAutomationReport::GetDisplayNameWithDecoration() const
{
	FString FinalDisplayName = TestInfo.GetDisplayName();
	//if this is an internal leaf node and the "decoration" name is being requested
	if (FilteredChildReports.Num())
	{
		int32 NumChildren = GetTotalNumFilteredChildren();
		//append on the number of child tests
		return TestInfo.GetDisplayName() + FString::Printf(TEXT(" (%d)"), NumChildren);
	}
	return FinalDisplayName;
}

int32 FAutomationReport::GetTotalNumChildren() const
{
	int32 Total = 0;
	for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
	{
		int ChildCount = ChildReports[ChildIndex]->GetTotalNumChildren();
		//Only count leaf nodes
		if (ChildCount == 0)
		{
			Total ++;
		}
		Total += ChildCount;
	}
	return Total;
}

int32 FAutomationReport::GetTotalNumFilteredChildren() const
{
	int32 Total = 0;
	for ( int32 ChildIndex = 0; ChildIndex < FilteredChildReports.Num(); ++ChildIndex )
	{
		int ChildCount = FilteredChildReports[ChildIndex]->GetTotalNumFilteredChildren();
		//Only count leaf nodes
		if ( ChildCount == 0 )
		{
			Total++;
		}
		Total += ChildCount;
	}
	return Total;
}

void FAutomationReport::GetEnabledTestNames(TArray<FString>& OutEnabledTestNames, FString CurrentPath) const
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	//if this is a leaf and this test is enabled
	if ((ChildReports.Num() == 0) && IsEnabled())
	{
		const FString FullTestName = CurrentPath.Len() > 0 ? CurrentPath.AppendChar(TCHAR('.')) + TestInfo.GetDisplayName() : TestInfo.GetDisplayName();
		OutEnabledTestNames.Add(FullTestName);
	}
	else
	{
		if( !CurrentPath.IsEmpty() )
		{
			CurrentPath += TEXT(".");
		}
		CurrentPath += TestInfo.GetDisplayName();
		//recurse through the hierarchy
		for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
		{
			ChildReports[ChildIndex]->GetEnabledTestNames(OutEnabledTestNames,CurrentPath);
		}
	}
	return;
}

void FAutomationReport::GetFilteredTestNames(TArray<FString>& OutFilteredTestNames, FString CurrentPath) const
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	// start from FilteredChildReports
	if (CurrentPath.IsEmpty())
	{
		for (int32 FilteredChildIndex = 0; FilteredChildIndex < FilteredChildReports.Num(); ++FilteredChildIndex)
		{
			FilteredChildReports[FilteredChildIndex]->GetFilteredTestNames(OutFilteredTestNames, TestInfo.GetDisplayName());
		}
	}
	else // then continue collecting all leaf nodes
	{
		//if this is a leaf collect full test name
		if (FilteredChildReports.Num() == 0)
		{
			const FString FullTestName = CurrentPath.Len() > 0 ? CurrentPath.AppendChar(TCHAR('.')) + TestInfo.GetDisplayName() : TestInfo.GetDisplayName();
			OutFilteredTestNames.Add(FullTestName);
		}
		else
		{
			if (!CurrentPath.IsEmpty())
			{
				CurrentPath += TEXT(".");
			}
			CurrentPath += TestInfo.GetDisplayName();
			//recurse through the hierarchy
			for (int32 ChildIndex = 0; ChildIndex < FilteredChildReports.Num(); ++ChildIndex)
			{
				FilteredChildReports[ChildIndex]->GetFilteredTestNames(OutFilteredTestNames, CurrentPath);
			}
		}
	}
	return;
}

void FAutomationReport::SetEnabledTests(const TArray<FString>& InEnabledTests, FString CurrentPath)
{
	if (ChildReports.Num() == 0)
	{
		//Find of the full name of this test and see if it is in our list
		const FString FullTestName = CurrentPath.Len() > 0 ? CurrentPath.AppendChar(TCHAR('.')) + TestInfo.GetDisplayName() : TestInfo.GetDisplayName();
		const bool bNewEnabled = InEnabledTests.Contains(FullTestName);
		SetEnabled(bNewEnabled);
	}
	else
	{
		if( !CurrentPath.IsEmpty() )
		{
			CurrentPath += TEXT(".");
		}
		CurrentPath += TestInfo.GetDisplayName();

		//recurse through the hierarchy
		for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
		{
			ChildReports[ChildIndex]->SetEnabledTests(InEnabledTests,CurrentPath);
		}

		//Parent nodes should be checked if all of its children are
		const int32 TotalNumChildern = GetTotalNumChildren();
		const int32 EnabledChildren = GetEnabledTestsNum();
		bEnabled = (TotalNumChildern == EnabledChildren);
	}
}

int32 FAutomationReport::GetEnabledTestsNum() const
{
	int32 Total = 0;
	//if this is a leaf and this test is enabled
	if ((ChildReports.Num() == 0) && IsEnabled())
	{
		Total++;
	}
	else
	{
		//recurse through the hierarchy
		for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
		{
			Total += ChildReports[ChildIndex]->GetEnabledTestsNum();
		}
	}
	return Total;
}

bool FAutomationReport::IsEnabled() const
{
	return bEnabled;
}

void FAutomationReport::SetEnabled(bool bShouldBeEnabled)
{
	bEnabled = bShouldBeEnabled;
	//set children to the same value
	for (int32 ChildIndex = 0; ChildIndex < FilteredChildReports.Num(); ++ChildIndex)
	{
		FilteredChildReports[ChildIndex]->SetEnabled(bShouldBeEnabled);
	}
}

void FAutomationReport::SetSupport(const int32 ClusterIndex)
{
	SupportFlags |= (1<<ClusterIndex);

	//ensure there is enough room in the array for status per platform
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	for (int32 i = 0; i <= ClusterIndex; ++i)
	{
		//Make sure we have enough results for a single pass
		TArray<FAutomationTestResults> AutomationTestResult;
		AutomationTestResult.Add( FAutomationTestResults() );
		Results.Add( AutomationTestResult );
	}
}

bool FAutomationReport::IsSupported(const int32 ClusterIndex) const
{
	return (SupportFlags & (1<<ClusterIndex)) ? true : false;
}


EAutomationTestFlags FAutomationReport::GetTestFlags( ) const
{
	return TestInfo.GetTestFlags();
}

FString FAutomationReport::GetSourceFile() const
{
	return TestInfo.GetSourceFile();
}

int32 FAutomationReport::GetSourceFileLine() const
{
	return TestInfo.GetSourceFileLine();
}

void FAutomationReport::SetTestFlags( const EAutomationTestFlags InTestFlags)
{
	TestInfo.AddTestFlags( InTestFlags );

	if ( InTestFlags == EAutomationTestFlags::SmokeFilter )
	{
		bEnabled = true;
	}
}

const bool FAutomationReport::IsParent()
{
	return bIsParent;
}

const bool FAutomationReport::IsSmokeTest()
{
	return !!(GetTestFlags() & EAutomationTestFlags::SmokeFilter);
}

bool FAutomationReport::SetFilter( TSharedPtr< AutomationFilterCollection > InFilter, const bool ParentPassedFilter )
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	//assume that this node and all its children fail to pass the filter test
	bool bSelfOrChildPassedFilter = false;

	//assume this node should not be expanded in the UI
	bNodeExpandInUI = false;

	//test for empty search string or matching search string
	bSelfPassesFilter = InFilter->PassesAllFilters( SharedThis( this ) );

	if ( IsParent() && ParentPassedFilter )
	{
		bSelfPassesFilter = true;
	}

	//clear the currently filtered tests array
	FilteredChildReports.Empty();

	//see if any children pass the filter
	for( int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex )
	{
		bool ThisChildPassedFilter = ChildReports[ChildIndex]->SetFilter( InFilter, bSelfPassesFilter );

		if( ThisChildPassedFilter )
		{
			if ( !ChildReports[ChildIndex]->IsParent() || ChildReports[ChildIndex]->GetFilteredChildren().Num() > 0 )
			{
				FilteredChildReports.Add(ChildReports[ChildIndex]);
			}
		}

		if ( bNodeExpandInUI == false && ThisChildPassedFilter == true )
		{
			// A child node has passed the filter, so we want to expand this node in the UI
			bNodeExpandInUI = true;
		}
	}

	//if we passed name, speed, and status tests
	if( bSelfPassesFilter || bNodeExpandInUI )
	{
		//Passed the filter!
		bSelfOrChildPassedFilter = true;
	}

	return bSelfOrChildPassedFilter;
}

TArray<TSharedPtr<IAutomationReport> >& FAutomationReport::GetFilteredChildren()
{
	return FilteredChildReports;
}

TArray<TSharedPtr<IAutomationReport> >& FAutomationReport::GetChildReports()
{
	return ChildReports;
}

void FAutomationReport::ClustersUpdated(const int32 NumClusters)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	TestInfo.ResetNumDevicesRunningTest();

	//Fixup Support flags
	SupportFlags = 0;
	for (int32 i = 0; i <= NumClusters; ++i)
	{
		SupportFlags |= (1<<i);
	}

	//Fixup Results array
	if( NumClusters > Results.Num() )
	{
		for( int32 ClusterIndex = Results.Num(); ClusterIndex < NumClusters; ++ClusterIndex )
		{
			//Make sure we have enough results for a single pass
			TArray<FAutomationTestResults> AutomationTestResult;
			AutomationTestResult.Add( FAutomationTestResults() );
			Results.Add( AutomationTestResult );
		}
	}
	else if( NumClusters < Results.Num() )
	{
		Results.RemoveAt(NumClusters, Results.Num() - NumClusters);
	}

	//recurse to children
	for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
	{
		ChildReports[ChildIndex]->ClustersUpdated(NumClusters);
	}
}

void FAutomationReport::ResetForExecution(const int32 NumTestPasses)
{
	TestInfo.ResetNumDevicesRunningTest();

	//if this test is enabled
	if (IsEnabled())
	{
		for (int32 ClusterIndex = 0; ClusterIndex < Results.Num(); ++ClusterIndex)
		{
			//Make sure we have enough results
			if( NumTestPasses > Results[ClusterIndex].Num() )
			{
				for(int32 PassCount = Results[ClusterIndex].Num(); PassCount < NumTestPasses; ++PassCount)
				{
					Results[ClusterIndex].Add( FAutomationTestResults() );
				}
			}
			else if( NumTestPasses < Results[ClusterIndex].Num() )
			{
				Results[ClusterIndex].RemoveAt(NumTestPasses, Results[ClusterIndex].Num() - NumTestPasses);
			}

			for( int32 PassIndex = 0; PassIndex < Results[ClusterIndex].Num(); ++PassIndex)
			{
				//reset all stats
				Results[ClusterIndex][PassIndex].Reset();
			}
		}
	}

	//recurse to children
	for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
	{
		ChildReports[ChildIndex]->ResetForExecution(NumTestPasses);
	}
}

void FAutomationReport::SetResults( const int32 ClusterIndex, const int32 PassIndex, const FAutomationTestResults& InResults )
{
	//verify this is a platform this test is aware of
	check((ClusterIndex >= 0) && (ClusterIndex < Results.Num()));
	check((PassIndex >= 0) && (PassIndex < Results[ClusterIndex].Num()));

	if( InResults.State == EAutomationState::InProcess )
	{
		TestInfo.InformOfNewDeviceRunningTest();
	}

	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	const TArray<FAutomationArtifact> ExistingArtifacts = Results[ClusterIndex][PassIndex].Artifacts;
	Results[ClusterIndex][PassIndex] = InResults;
	Results[ClusterIndex][PassIndex].Artifacts.Append(ExistingArtifacts);

	// Add an error report if none was received
	if ( InResults.State == EAutomationState::Fail && InResults.GetErrorTotal() == 0 )
	{
		FString Msg = FString::Printf(TEXT("Test %s failed, but no errors were logged."), *TestInfo.GetFullTestPath());
		Results[ClusterIndex][PassIndex].AddEvent(FAutomationEvent(EAutomationEventType::Error, *Msg));
	}

	// While setting the results of the test cause the log of any selected test to refresh
	OnSetResults.ExecuteIfBound(AsShared());
}

void FAutomationReport::AddArtifact(const int32 ClusterIndex, const int32 PassIndex, const FAutomationArtifact& Artifact)
{
	//verify this is a platform this test is aware of
	check(( ClusterIndex >= 0 ) && ( ClusterIndex < Results.Num() ));
	check(( PassIndex >= 0 ) && ( PassIndex < Results[ClusterIndex].Num() ));

	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
	Results[ClusterIndex][PassIndex].Artifacts.Add(Artifact);
}

void FAutomationReport::GetCompletionStatus(const int32 ClusterIndex, const int32 PassIndex, FAutomationCompleteState& OutCompletionState)
{
	//if this test is enabled and a leaf test
	if (IsSupported(ClusterIndex) && (ChildReports.Num()==0))
	{
		EAutomationState CurrentState = Results[ClusterIndex][PassIndex].State;
		//Enabled and In-Process
		if (IsEnabled())
		{
			OutCompletionState.TotalEnabled++;
			if (CurrentState == EAutomationState::InProcess)
			{
				OutCompletionState.NumEnabledInProcess++;
			}
		}

		//Warnings
		if (Results[ClusterIndex][PassIndex].GetWarningTotal() > 0)
		{
			IsEnabled() ? OutCompletionState.NumEnabledTestsWarnings++ : OutCompletionState.NumDisabledTestsWarnings++;
		}

		//Test results
		if (CurrentState == EAutomationState::Success)
		{
			IsEnabled() ? OutCompletionState.NumEnabledTestsPassed++ : OutCompletionState.NumDisabledTestsPassed++;
		}
		else if (CurrentState == EAutomationState::Fail)
		{
			IsEnabled() ? OutCompletionState.NumEnabledTestsFailed++ : OutCompletionState.NumDisabledTestsFailed++;
		}
		else if( CurrentState == EAutomationState::Skipped )
		{
			IsEnabled() ? OutCompletionState.NumEnabledTestsCouldntBeRun++ : OutCompletionState.NumDisabledTestsCouldntBeRun++;
		}
	}
	//recurse to children
	for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
	{
		ChildReports[ChildIndex]->GetCompletionStatus(ClusterIndex,PassIndex, OutCompletionState);
	}
}


EAutomationState FAutomationReport::GetState(const int32 ClusterIndex, const int32 PassIndex) const
{
	if ((ClusterIndex >= 0) && (ClusterIndex < Results.Num()) &&
		(PassIndex >= 0) && (PassIndex < Results[ClusterIndex].Num()))
	{
		return Results[ClusterIndex][PassIndex].State;
	}
	return EAutomationState::NotRun;
}

void FAutomationReport::SetState(const EAutomationState State)
{
	if (IsEnabled())
	{
		for (int32 ResultsIndex = 0; ResultsIndex < Results.Num(); ++ResultsIndex)
		{
			for (int32 PassIndex = 0; PassIndex < Results[ResultsIndex].Num(); ++PassIndex)
			{
				Results[ResultsIndex][PassIndex].State = State;
			}
		}
	}
}

const FAutomationTestResults& FAutomationReport::GetResults( const int32 ClusterIndex, const int32 PassIndex ) 
{
	return Results[ClusterIndex][PassIndex];
}

const int32 FAutomationReport::GetNumResults( const int32 ClusterIndex )
{
	return Results[ClusterIndex].Num();
}

const int32 FAutomationReport::GetCurrentPassIndex( const int32 ClusterIndex )
{
	int32 PassIndex = 1;

	if( IsSupported(ClusterIndex) )
	{
		for(; PassIndex < Results[ClusterIndex].Num(); ++PassIndex )
		{
			if( Results[ClusterIndex][PassIndex].State == EAutomationState::NotRun )
			{
				break;
			}
		}
	}

	return PassIndex - 1;
}

FString FAutomationReport::GetGameInstanceName( const int32 ClusterIndex )
{
	return Results[ClusterIndex][0].GameInstance;
}

TSharedPtr<IAutomationReport> FAutomationReport::EnsureReportExists(FAutomationTestInfo& InTestInfo, const int32 ClusterIndex, const int32 NumPasses)
{
	//Split New Test Name by the first "." found
	FString NameToMatch = InTestInfo.GetDisplayName();
	FString FullPath = InTestInfo.GetFullTestPath();
	FString NameRemainder;
	//if this is a leaf test (no ".")
	if (!InTestInfo.GetDisplayName().Split(TEXT("."), &NameToMatch, &NameRemainder))
	{
		NameToMatch = InTestInfo.GetDisplayName();
	}

	if ( NameRemainder.Len() != 0 )
	{
		// Set the test info name to be the remaining string
		InTestInfo.SetDisplayName( NameRemainder );
		// Update the fullpath
		FullPath.LeftChopInline(NameRemainder.Len() + 1);
	}

	uint32 NameToMatchHash = GetTypeHash(FullPath);

	TSharedPtr<IAutomationReport> MatchTest;
	//check hash table first to see if it exists yet
	if (ChildReportNameHashes.Contains(NameToMatchHash))
	{
		//go backwards.  Most recent event most likely matches
		int32 TestIndex = ChildReports.Num() - 1;
		for (; TestIndex >= 0; --TestIndex)
		{
			//if the name matches
			if (ChildReports[TestIndex]->GetFullTestPath() == FullPath)
			{
				MatchTest = ChildReports[TestIndex];
				break;
			}
		}
	}

	//if there isn't already a test like this
	if (!MatchTest.IsValid())
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Report"));
		if ( NameRemainder.Len() == 0 )
		{
			// Create a new leaf node
			MatchTest = MakeShareable(new FAutomationReport(InTestInfo));			
		}
		else
		{
			// Create a parent node
			FAutomationTestInfo ParentTestInfo(NameToMatch, FullPath, TEXT(""), InTestInfo.GetTestFlags(), InTestInfo.GetNumParticipantsRequired());
			MatchTest = MakeShareable(new FAutomationReport(ParentTestInfo, true));
		}

		// Sort tests alphabetically
		const int32 InsertAt = Algo::UpperBound<>(ChildReports, MatchTest, [](const TSharedPtr<IAutomationReport>& ReportA, const TSharedPtr<IAutomationReport>& ReportB)
			{
				bool AIsLeafNode = !ReportA->IsParent();
				bool BIsLeafNode = !ReportB->IsParent();

				if (AIsLeafNode == BIsLeafNode) // both leaves or both parents => normal comparison
				{
					return ReportA->GetDisplayName() < ReportB->GetDisplayName();
				}
				else // leaf and parent => A is less than B when B is the leaf
				{
					return BIsLeafNode;
				}
			}
		);

		// Insert new test (sorted)
		ChildReports.Insert(MatchTest, InsertAt);
		ChildReportNameHashes.Add(NameToMatchHash, NameToMatchHash);
	}

	//mark this test as supported on a particular platform
	MatchTest->SetSupport(ClusterIndex);

	MatchTest->SetTestFlags( InTestInfo.GetTestFlags() );
	MatchTest->SetNumParticipantsRequired( MatchTest->GetNumParticipantsRequired() > InTestInfo.GetNumParticipantsRequired() ? MatchTest->GetNumParticipantsRequired() : InTestInfo.GetNumParticipantsRequired() );

	TSharedPtr<IAutomationReport> FoundTest;
	//if this is a leaf node
	if (NameRemainder.Len() == 0)
	{
		FoundTest = MatchTest;
	}
	else
	{
		//recurse to add to the proper layer
		FoundTest = MatchTest->EnsureReportExists(InTestInfo, ClusterIndex, NumPasses);
	}

	return FoundTest;
}


TSharedPtr<IAutomationReport> FAutomationReport::GetNextReportToExecute(bool& bOutAllTestsComplete, const int32 ClusterIndex, const int32 PassIndex, const int32 NumDevicesInCluster)
{
	TSharedPtr<IAutomationReport> NextReport;
	//if this is not a leaf node
	if (ChildReports.Num())
	{
		for (int32 ReportIndex = 0; ReportIndex < ChildReports.Num(); ++ReportIndex)
		{
			NextReport = ChildReports[ReportIndex]->GetNextReportToExecute(bOutAllTestsComplete, ClusterIndex, PassIndex, NumDevicesInCluster);
			//if we found one, return it
			if (NextReport.IsValid())
			{
				break;
			}
		}
	}
	else
	{
		//consider self
		if (IsEnabled() && IsSupported(ClusterIndex))
		{
			EAutomationState TestState = GetState(ClusterIndex,PassIndex);
			//if any enabled test hasn't been run yet or is in process
			if (TestState == EAutomationState::NotRun || TestState == EAutomationState::InProcess)
			{
				//make sure we announce we are NOT done with all tests
				bOutAllTestsComplete = false;
			}
			if (TestState == EAutomationState::NotRun)
			{
				//Found one to run next
				NextReport = AsShared();
			}
		}
	}
	return NextReport;
}

void FAutomationReport::GetEnabledTestReports(TArray<TSharedPtr<IAutomationReport>>& OutReports)
{
	//if this is not a leaf node
	if (ChildReports.Num())
	{
		for (int32 ReportIndex = 0; ReportIndex < ChildReports.Num(); ++ReportIndex)
		{
			ChildReports[ReportIndex]->GetEnabledTestReports(OutReports);
		}
	}
	else
	{
		if (IsEnabled())
		{
			//Found one to run
			OutReports.Add(AsShared());
		}
	}
}

const bool FAutomationReport::HasErrors()
{
	bool bHasErrors = false;
	for ( int32 ClusterIndex = 0; ClusterIndex < Results.Num(); ++ClusterIndex )
	{
		for ( int32 PassIndex = 0; PassIndex < Results[ClusterIndex].Num(); ++PassIndex )
		{
			//if we want tests with errors and this test had them OR we want tests warnings and this test had them
			if ( Results[ClusterIndex][PassIndex].GetErrorTotal() > 0 )
			{
				//mark this test as having passed the results filter
				bHasErrors = true;
				break;
			}
		}
	}
	return bHasErrors;
}

const bool FAutomationReport::HasWarnings()
{
	bool bHasWarnings = false;
	for ( int32 ClusterIndex = 0; ClusterIndex < Results.Num(); ++ClusterIndex )
	{
		for ( int32 PassIndex = 0; PassIndex < Results[ClusterIndex].Num(); ++PassIndex )
		{
			//if we want tests with errors and this test had them OR we want tests warnings and this test had them
			if ( Results[ClusterIndex][PassIndex].GetWarningTotal() > 0 )
			{
				//mark this test as having passed the results filter
				bHasWarnings = true;
				break;
			}
		}
	}
	return bHasWarnings;
}

const bool FAutomationReport::GetDurationRange(float& OutMinTime, float& OutMaxTime)
{
	//assume we haven't found any tests that have completed successfully
	OutMinTime = MAX_FLT;
	OutMaxTime = 0.0f;
	bool bAnyResultsFound = false;

	//keep sum of all child tests
	float ChildTotalMinTime = 0.0f;
	float ChildTotalMaxTime = 0.0f;
	for (int32 ReportIndex = 0; ReportIndex < ChildReports.Num(); ++ReportIndex)
	{
		float ChildMinTime = MAX_FLT;
		float ChildMaxTime = 0.0f;
		if (ChildReports[ReportIndex]->GetDurationRange(ChildMinTime, ChildMaxTime))
		{
			ChildTotalMinTime += ChildMinTime;
			ChildTotalMaxTime += ChildMaxTime;
			bAnyResultsFound = true;
		}
	}

	//if any child test had valid timings
	if (bAnyResultsFound)
	{
		OutMinTime = ChildTotalMinTime;
		OutMaxTime = ChildTotalMaxTime;
	}

	for (int32 ClusterIndex = 0; ClusterIndex < Results.Num(); ++ClusterIndex )
	{
		for( int32 PassIndex = 0; PassIndex < Results[ClusterIndex].Num(); ++PassIndex)
		{
			//if we want tests with errors and this test had them OR we want tests warnings and this test had them
			if( Results[ClusterIndex][PassIndex].State == EAutomationState::Success ||
				Results[ClusterIndex][PassIndex].State == EAutomationState::Fail)
			{
				OutMinTime = FMath::Min(OutMinTime, Results[ClusterIndex][PassIndex].Duration );
				OutMaxTime = FMath::Max(OutMaxTime, Results[ClusterIndex][PassIndex].Duration );
				bAnyResultsFound = true;
			}
		}
	}
	return bAnyResultsFound;
}


const int32 FAutomationReport::GetNumDevicesRunningTest() const
{
	return TestInfo.GetNumDevicesRunningTest();
}


const int32 FAutomationReport::GetNumParticipantsRequired() const
{
	return TestInfo.GetNumParticipantsRequired();
}


void FAutomationReport::SetNumParticipantsRequired( const int32 NewCount )
{
	TestInfo.SetNumParticipantsRequired( NewCount );
}


bool FAutomationReport::IncrementNetworkCommandResponses()
{
	NumberNetworkResponsesReceived++;
	return (NumberNetworkResponsesReceived == TestInfo.GetNumParticipantsRequired());
}


void FAutomationReport::ResetNetworkCommandResponses()
{
	NumberNetworkResponsesReceived = 0;
}


const bool FAutomationReport::ExpandInUI() const
{
	return bNodeExpandInUI;
}


void FAutomationReport::StopRunningTest()
{
	if( IsEnabled() )
	{
		for( int32 ResultsIndex = 0; ResultsIndex < Results.Num(); ++ResultsIndex )
		{
			for( int32 PassIndex = 0; PassIndex < Results[ResultsIndex].Num(); ++PassIndex)
			{
				if( Results[ResultsIndex][PassIndex].State == EAutomationState::InProcess )
				{
					Results[ResultsIndex][PassIndex].State = EAutomationState::NotRun;
				}
			}
		}
	}

	// Recurse to children
	for( int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex )
	{
		ChildReports[ChildIndex]->StopRunningTest();
	}
}

bool FAutomationReport::IsToBeSkipped(FName* OutReason, bool* OutWarn) const
{
	if (bNeedToSkip)
	{
		if (OutReason != nullptr)
		{
			if (!ExcludeTestInfo.Platforms.IsEmpty())
			{
				*OutReason = *(SetToShortString(ExcludeTestInfo.Platforms) + TEXT(": ") + ExcludeTestInfo.Reason.ToString());
			}
			else if (!ExcludeTestInfo.RHIs.IsEmpty())
			{
				*OutReason = *(SetToShortString(ExcludeTestInfo.RHIs) + TEXT(": ") + ExcludeTestInfo.Reason.ToString());
			}
			else
			{
				*OutReason = ExcludeTestInfo.Reason;
			}
		}

		if (OutWarn != nullptr)
		{
			*OutWarn = ExcludeTestInfo.Warn;
		}

		return true;
	}

	return false;
}

bool FAutomationReport::IsToBeSkippedOnConditions() const
{
	return bNeedToSkip && ExcludeTestInfo.HasConditions();
}

bool FAutomationReport::IsToBeSkippedByPropagation() const
{
	return bNeedToSkip && ExcludeTestInfo.bIsPropagated;
}

void FAutomationReport::SetSkipFlag(bool bEnableSkip, const FAutomationTestExcludelistEntry* Template, bool bFromPropagation)
{
	if (IsToBeSkipped() == bEnableSkip)
	{
		if (!bEnableSkip || Template == nullptr)
			return;

		if (!bFromPropagation)
		{
			// Remove previous entry in the config
			UAutomationTestExcludelist::Get()->RemoveFromExcludeTest(TestInfo.GetFullTestPath());
		}
	}

	if (!bFromPropagation && !ExcludeTestInfo.IsEmpty() && ExcludeTestInfo.bIsPropagated && !ExcludeTestInfo.HasConditions())
		return; // Propagated exclusion can't be changed directly

	bNeedToSkip = bEnableSkip;

	if (Template != nullptr)
	{
		// Update the entry
		ExcludeTestInfo = *Template;
		ExcludeTestInfo.bIsPropagated = bFromPropagation;
	}

	auto ExcludedTestCached = UAutomationTestExcludelist::Get();
	if (bFromPropagation)
	{
		auto Entry = ExcludedTestCached->GetExcludeTestEntry(TestInfo.GetFullTestPath());
		if (bNeedToSkip)
		{
			// If we get an exclusion entry, check if it is the original one.
			if (Entry != nullptr)
			{
				ExcludeTestInfo = *Entry;
				ExcludeTestInfo.SetPropagation(TestInfo.GetFullTestPath());
			}
		}
		else
		{
			// Before enabling the test, check if there is an underlying exclusion already set
			if (Entry == nullptr)
			{
				ExcludeTestInfo.Reset();
			}
			else
			{
				// Update instance exclusion info
				ExcludeTestInfo = *Entry;
				bNeedToSkip = true;
				// Update exclusion template for the children propagation
				Template = Entry;
			}
		}
	}
	else
	{
		if (bNeedToSkip)
		{
			check(Template != nullptr);
			auto Entry = ExcludedTestCached->GetExcludeTestEntry(TestInfo.GetFullTestPath());
			if (Entry != nullptr && ExcludeTestInfo.RemoveConditions(*Entry))
			{
				// Branch off the template with the difference exclusion condition set
				// Remove the overlapping conditions as they are redundant since the parent has precedence 
				Template = &ExcludeTestInfo;
			}
			ExcludedTestCached->AddToExcludeTest(TestInfo.GetFullTestPath(), *Template);
		}
		else
		{
			ExcludedTestCached->RemoveFromExcludeTest(TestInfo.GetFullTestPath());
			auto Entry = ExcludedTestCached->GetExcludeTestEntry(TestInfo.GetFullTestPath());
			if (Entry != nullptr)
			{
				// If there is still an entry, it means a higher exclusion rule exists so we apply it.
				// Update instance exclusion info
				ExcludeTestInfo = *Entry;
				ExcludeTestInfo.bIsPropagated = true;
				bNeedToSkip = true;
				// Update exclusion template for the children propagation
				Template = Entry;
			}
		}
	}

	// Propagate to children
	if (IsParent())
	{
		for (IAutomationReportPtr Child : GetChildReports())
		{
			Child->SetSkipFlag(bNeedToSkip, Template, true);
		}
	}

	if (!bFromPropagation)
	{
		// Save config only at the end of the recursion
		ExcludedTestCached->SaveToConfigs();
	}
}

TSharedPtr<FAutomationTestExcludeOptions> FAutomationReport::GetExcludeOptions()
{
	ExcludeTestInfo.Test = *TestInfo.GetFullTestPath();
	return ExcludeTestInfo.GetOptions();
}

