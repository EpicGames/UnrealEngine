// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "AutomationTestExcludelist.h"
#include "IAutomationReport.h"

/**
 * Interface for automation test results modules.
 */
class FAutomationReport : public IAutomationReport
{
public:

	FAutomationReport(FAutomationTestInfo& TestInfo, bool bIsParent = false);

public:

	// IAutomationReport Interface

	virtual void Empty() override;
	virtual FString GetTestParameter() const override;
	virtual FString GetTags() const override;
	virtual TArray<FString> GetTagsArray() const override;
	virtual FString GetAssetPath() const override;
	virtual FString GetOpenCommand() const override;
	virtual FString GetCommand() const override;
	virtual const FString& GetDisplayName() const override;
	virtual const FString& GetFullTestPath() const override;
	virtual FString GetDisplayNameWithDecoration() const override;
	virtual int32 GetTotalNumChildren() const override;
	virtual int32 GetTotalNumFilteredChildren() const override;
	virtual int32 GetEnabledTestsNum() const override;
	virtual void GetEnabledTestNames(TArray<FString>& OutEnabledTestNames, FString CurrentPath) const override;
	virtual void GetFilteredTestNames(TArray<FString>& OutFilteredTestNames, FString CurrentPath) const override;
	virtual void SetEnabledTests(const TArray<FString>& EnabledTests, FString CurrentPath) override;
	virtual bool IsEnabled() const override;
	virtual void SetEnabled(bool bShouldBeEnabled) override;
	virtual void SetSupport(const int32 ClusterIndex) override;
	virtual bool IsSupported(const int32 ClusterIndex) const override;
	virtual void SetTestFlags(const EAutomationTestFlags TestFlags) override;
	virtual EAutomationTestFlags GetTestFlags( ) const override;
	virtual FString GetSourceFile() const override;
	virtual int32 GetSourceFileLine() const override;
	virtual const bool IsParent( ) override;
	virtual const bool IsSmokeTest( ) override;
	virtual bool SetFilter( TSharedPtr< AutomationFilterCollection > InFilter, const bool ParentPassedFilter = false ) override;
	virtual TArray<TSharedPtr<IAutomationReport> >& GetFilteredChildren() override;
	virtual TArray<TSharedPtr<IAutomationReport> >& GetChildReports() override;
	virtual void ClustersUpdated(const int32 NumClusters) override;
	virtual void ResetForExecution(const int32 NumTestPasses) override;
	virtual void SetResults(const int32 ClusterIndex, const int32 PassIndex, const FAutomationTestResults& InResults) override;
	virtual void AddArtifact(const int32 ClusterIndex, const int32 PassIndex, const FAutomationArtifact& Artifact) override;
	virtual void GetCompletionStatus(const int32 ClusterIndex, const int32 PassIndex, FAutomationCompleteState& OutCompletionState) override;
	virtual EAutomationState GetState(const int32 ClusterIndex, const int32 PassIndex) const override;
	virtual void SetState(const EAutomationState State) override;
	virtual const FAutomationTestResults& GetResults( const int32 ClusterIndex, const int32 PassIndex ) override;
	virtual const int32 GetNumResults( const int32 ClusterIndex ) override;
	virtual const int32 GetCurrentPassIndex( const int32 ClusterIndex ) override;
	virtual FString GetGameInstanceName( const int32 ClusterIndex ) override;
	virtual TSharedPtr<IAutomationReport> EnsureReportExists(FAutomationTestInfo& TestInfo, const int32 ClusterIndex, const int32 NumPasses) override;
	virtual TSharedPtr<IAutomationReport> GetNextReportToExecute(bool& bOutAllTestsComplete, const int32 ClusterIndex, const int32 PassIndex, const int32 NumDevicesInCluster) override;
	virtual void GetEnabledTestReports(TArray<TSharedPtr<IAutomationReport>>& OutReports) override;
	virtual const bool HasErrors() override;
	virtual const bool HasWarnings() override;
	virtual const bool GetDurationRange(float& OutMinTime, float& OutMaxTime) override;
	virtual const int32 GetNumDevicesRunningTest() const override;
	virtual const int32 GetNumParticipantsRequired() const override;
	virtual void SetNumParticipantsRequired( const int32 NewCount ) override;
	virtual bool IncrementNetworkCommandResponses() override;
	virtual void ResetNetworkCommandResponses() override;
	virtual const bool ExpandInUI() const override;
	virtual void StopRunningTest() override;
	virtual bool IsToBeSkipped(FName* OutReason = nullptr, bool* OutWarn = nullptr) const override;
	virtual bool IsToBeSkippedOnConditions() const override;
	virtual bool IsToBeSkippedByPropagation() const override;
	virtual void SetSkipFlag(bool bEnableSkip, const FAutomationTestExcludelistEntry* Template = nullptr, bool bFromPropagation = false) override;
	virtual TSharedPtr<FAutomationTestExcludeOptions> GetExcludeOptions() override;

private:

	/** True if this test should be executed */
	bool bEnabled;

	/** True if this test is a parent */
	bool bIsParent;

	/** True if this report should be expanded in the UI */
	bool bNodeExpandInUI;

	/** True if this report has passed the filter and should be highlighted in the UI */
	bool bSelfPassesFilter;

	/** List of bits that represents which device types requested this test */
	uint32 SupportFlags;

	/** Number of responses from network commands */
	uint32 NumberNetworkResponsesReceived;

	/** All child tests */
	TArray<TSharedPtr<IAutomationReport> >ChildReports;

	/** Map of all Report Name hashes to avoid iterating all items to test for existance*/
	TMap<uint32, uint32> ChildReportNameHashes;

	/** Filtered child tests */
	TArray<TSharedPtr<IAutomationReport> >FilteredChildReports;

	/** Results from execution of the test (per cluster) */
	TArray< TArray<FAutomationTestResults> > Results;

	/** Structure holding the test info */
	FAutomationTestInfo TestInfo;

	/** Structure holding the info on the exclude test */
	FAutomationTestExcludelistEntry ExcludeTestInfo;
	
	/** True if this item is inside the exclude list */
	bool bNeedToSkip = false;
};
