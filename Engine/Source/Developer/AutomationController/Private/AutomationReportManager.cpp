// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationReportManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "IAutomationControllerModule.h"
#include "AutomationReport.h"

namespace
{
	// Note that if you change at least one line here you must change other lines in order to keep consistent columns count.
	// Header string to add as a report's first line
	constexpr TCHAR ReportHeader[] = TEXT("\"GameInstance\",\"Platform\",\"Test Name\",\"Duration\",\"Status\"");
	// Format string that is used to fill the lines that contain report's data
	constexpr TCHAR ReportLineFormatString[] = TEXT("\"%s\",\"%s\",\"%s\",\"%s\",\"%s\"");
	// Pre and post fix strings to add to the errors and warnings for viewing in CSV. Spaces correctly, and allows commas in the message
	constexpr TCHAR ErrorReportLinePrefixText[] = TEXT("\t,,,,\"");
	constexpr TCHAR ErrorReportLineSufixText[] = TEXT("\"");
} // anonymous namespace


FAutomationReportManager::FAutomationReportManager()
{
	//ensure that we have a valid root to the hierarchy
	FAutomationTestInfo TestInfo;
	ReportRoot = MakeShareable(new FAutomationReport( TestInfo ));
}


void FAutomationReportManager::Empty()
{
	//ensure there is a root node
	ReportRoot->Empty();
}


void FAutomationReportManager::ClustersUpdated( const int32 NumClusters )
{
	ReportRoot->ClustersUpdated(NumClusters);
}


void FAutomationReportManager::ResetForExecution(const int32 NumTestPasses)
{
	//Recursively prepare all tests for execution
	ReportRoot->ResetForExecution(NumTestPasses);
}


void FAutomationReportManager::StopRunningTests()
{
	ReportRoot->StopRunningTest();
}


TSharedPtr<IAutomationReport> FAutomationReportManager::GetNextReportToExecute(bool& bOutAllTestsComplete, const int32 ClusterIndex, const int32 PassIndex, const int32 NumDevicesInCluster)
{
	TSharedPtr<IAutomationReport> ReportToExecute = ReportRoot->GetNextReportToExecute(bOutAllTestsComplete, ClusterIndex, PassIndex, NumDevicesInCluster);
	return ReportToExecute;
}

TArray<TSharedPtr<IAutomationReport>> FAutomationReportManager::GetEnabledTestReports()
{
	TArray<TSharedPtr<IAutomationReport>> Reports;
	ReportRoot->GetEnabledTestReports(Reports);
	return Reports;
}

TSharedPtr<IAutomationReport> FAutomationReportManager::EnsureReportExists(FAutomationTestInfo& TestInfo, const int32 ClusterIndex, const int32 NumPasses)
{
	TSharedPtr<IAutomationReport> NewStatus = ReportRoot->EnsureReportExists (TestInfo, ClusterIndex, NumPasses);
	return NewStatus;
}


void FAutomationReportManager::SetFilter( TSharedPtr< AutomationFilterCollection > InFilter )
{
	ReportRoot->SetFilter( InFilter );
}


TArray <TSharedPtr <IAutomationReport> >& FAutomationReportManager::GetFilteredReports()
{
	return ReportRoot->GetFilteredChildren();
}


void FAutomationReportManager::SetVisibleTestsEnabled(const bool bEnabled)
{
	ReportRoot->SetEnabled(bEnabled);
}


int32 FAutomationReportManager::GetEnabledTestsNum() const
{
	return ReportRoot->GetEnabledTestsNum();
}


void FAutomationReportManager::GetEnabledTestNames(TArray<FString>& OutEnabledTestNames) const
{
	ReportRoot->GetEnabledTestNames(OutEnabledTestNames,FString());
}

void FAutomationReportManager::GetFilteredTestNames(TArray<FString>& OutFilteredTestNames) const
{
	ReportRoot->GetFilteredTestNames(OutFilteredTestNames, FString());
}

void FAutomationReportManager::SetEnabledTests(const TArray<FString>& EnabledTests)
{
	ReportRoot->SetEnabledTests(EnabledTests,FString());
}


const bool FAutomationReportManager::ExportReport(uint32 FileExportTypeMask, const int32 NumDeviceClusters)
{
	// The results log. Errors, warnings etc are added to this
	TArray<FString> ResultsLog = { ReportHeader };

	// Create the report be recursively going through the results tree
	FindLeafReport(ReportRoot, NumDeviceClusters, ResultsLog, FileExportTypeMask);

	// Has a report been generated
	bool ReportGenerated = false;

	// Ensure we have a log to write (the first line is a header line)
	if (ResultsLog.Num() > 1)
	{
		// Create the file name
		FString FileName = FString::Printf( TEXT( "Automation%s.csv" ), *FDateTime::Now().ToString() );
		FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::AutomationDir());
		FString FullPath = FString::Printf( TEXT( "%s%s" ), *FileLocation, *FileName );

		// save file
		FArchive* LogFile = IFileManager::Get().CreateFileWriter( *FullPath );

		if (LogFile != NULL)
		{
			for (int32 Index = 0; Index < ResultsLog.Num(); ++Index)
			{
				FString LogEntry = FString::Printf( TEXT( "%s" ),
					*ResultsLog[ Index ]) + LINE_TERMINATOR;
				LogFile->Serialize( TCHAR_TO_ANSI( *LogEntry ), LogEntry.Len() );
			}

			LogFile->Close();
			delete LogFile;

			// A report has been generated
			ReportGenerated = true;
		}
	}

	return ReportGenerated;
}

void FAutomationReportManager::AddResultReport(TSharedPtr< IAutomationReport > InReport, const int32 NumDeviceClusters, TArray< FString >& ResultsLog, uint32 FileExportTypeMask)
{
	if (InReport->IsEnabled())
	{
		for (int32 ClusterIndex = 0; ClusterIndex < NumDeviceClusters; ++ClusterIndex)
		{
			FAutomationTestResults TestResults = InReport->GetResults( ClusterIndex,CurrentTestPass );

			// Early out if we don't want this report
			bool AddReport = true;
			if  ( !EFileExportType::IsSet( FileExportTypeMask, EFileExportType::FET_All ) && !EFileExportType::IsSet( FileExportTypeMask, EFileExportType::FET_Status ) )
			{
				uint32 ResultMask = 0;

				if (TestResults.GetErrorTotal() > 0)
				{
					EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Errors );
				}
				if (TestResults.GetWarningTotal() > 0)
				{
					EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Warnings );
				}
				if (TestResults.GetLogTotal())
				{
					EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Logs );
				}

				// Ensure we have a report that passes at least one filter
				if ( ( ResultMask & FileExportTypeMask ) == 0 )
				{
					AddReport = false;
				}
			}

			if (AddReport)
			{
				// Get the duration of the test
				float MinDuration;
				float MaxDuration;
				FString DurationString;
				if (InReport->GetDurationRange(MinDuration, MaxDuration ))
				{
					//if there is a duration range
					if (MinDuration != MaxDuration)
					{
						DurationString = FString::Printf( TEXT( "%4.4fs - %4.4fs" ), MinDuration, MaxDuration );
					}
					else
					{
						DurationString = FString::Printf( TEXT( "%4.4fs" ), MinDuration );
					}
				}

				// Build a status string that contains information about the test 
				FString Status = AutomationStateToString(TestResults.State);

				// Create the log string
				FString Info = FString::Printf( ReportLineFormatString,
					*InReport->GetGameInstanceName( ClusterIndex ),
					*FModuleManager::GetModuleChecked< IAutomationControllerModule >( "AutomationController" ).GetAutomationController()->GetDeviceTypeName( ClusterIndex ),
					*InReport->GetDisplayNameWithDecoration(),
					*DurationString,
					*Status );

				ResultsLog.Add( Info );

				// Add any warning or errors
				for ( const FAutomationExecutionEntry& Entry : TestResults.GetEntries() )
				{
					switch (Entry.Event.Type )
					{
					case EAutomationEventType::Info:
						if ( EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_All) || EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_Logs) )
						{
							ResultsLog.Add(ErrorReportLinePrefixText + Entry.ToString() + ErrorReportLineSufixText);
						}
						break;
					case EAutomationEventType::Warning:
						if ( EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_All) || EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_Warnings) )
						{
							ResultsLog.Add(ErrorReportLinePrefixText + Entry.ToString() + ErrorReportLineSufixText);
						}
						break;
					case EAutomationEventType::Error:
						if ( EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_All) || EFileExportType::IsSet(FileExportTypeMask, EFileExportType::FET_Errors) )
						{
							ResultsLog.Add(ErrorReportLinePrefixText + Entry.ToString() + ErrorReportLineSufixText);
						}
						break;
					}
				}
			}
		}
	}
}


void FAutomationReportManager::FindLeafReport(TSharedPtr< IAutomationReport > InReport, const int32 NumDeviceClusters, TArray< FString >& ResultsLog, uint32 FileExportTypeMask)
{
	TArray< TSharedPtr< IAutomationReport > >& ChildReports = InReport->GetFilteredChildren();

	// If there are no child reports, we have reached a leaf that has had a test run on it.
	if (ChildReports.Num() == 0)
	{
		AddResultReport( InReport, NumDeviceClusters, ResultsLog, FileExportTypeMask );
	}
	else
	{
		// We still have some child nodes. We won't have run a test on this node
		for (int32 ChildIndex = 0; ChildIndex < ChildReports.Num(); ++ChildIndex)
		{
			FindLeafReport( ChildReports[ ChildIndex ], NumDeviceClusters, ResultsLog, FileExportTypeMask );
		}
	}
}
