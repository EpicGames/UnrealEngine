// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAutomationReport.h"

/**
 * Wrapper class to obfuscate the hierarchy of tests
 */
class FAutomationReportManager
{
public:
	/** default constructor */
	FAutomationReportManager();

	/** Clears out all reports */
	void Empty();

	/** Updates the automation reports when the number of clusters changes */
	void ClustersUpdated( const int32 NumClusters );

	/** 
	 * Resets for the next run of tests 
	 * @param NumTestPasses The number of test results to store
	 */
	void ResetForExecution(const int32 NumTestPasses);

	/** Stops all tests from running */
	void StopRunningTests();

	/**
	 * Searches for the next test to execute.
	 *
	 * @param bOutAllTestsComplete Returns True if all tests for this device cluster have been completed.
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param PassIndex Index of the current test pass we are on.
	 * @param NumDevicesInCluster The number of devices which are available in this cluster.
	 */
	TSharedPtr<IAutomationReport> GetNextReportToExecute(bool& bOutAllTestsComplete, const int32 ClusterIndex, const int32 PassIndex, const int32 NumDevicesInCluster);

	/**
	 * Returns all the enabled test reports.
	 *
	 * @return an array of all reports
	 */
	TArray<TSharedPtr<IAutomationReport>> GetEnabledTestReports();

	/**
	 * Ensures the nested tree exists ("blueprint.test.all") would have 3 levels other than the root.
	 *
	 * @param TestInfo Structure containing the test info.
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param NumPasses The number of test passes to create reports for.
	 * @return The automation report.
	 */
	TSharedPtr<IAutomationReport> EnsureReportExists(FAutomationTestInfo& TestInfo, const int32 ClusterIndex, const int32 NumPasses);

	/** Filters the visible tests based on name, status, speed. */
	void SetFilter( TSharedPtr< AutomationFilterCollection > InFilter );
	
	/** gets array of filtered tests to display in the UI. */
	TArray <TSharedPtr <IAutomationReport> >& GetFilteredReports();

	/** Sets whether all visible tests are enabled or not. */
	void SetVisibleTestsEnabled(const bool bEnabled);

	/** Returns number of tests that will be run. */
	int32 GetEnabledTestsNum () const;

	/** Gets the names of enabled tests. */
	void GetEnabledTestNames(TArray<FString>& OutEnabledTestNames) const;

	/** Gets the names of filtered tests. */
	void GetFilteredTestNames(TArray<FString>& OutFilteredTestNames) const;

	/** Sets the enabled tests based off the passed list of enabled tests. */
	void SetEnabledTests(const TArray<FString>& EnabledTests);

	/** Sets the current test pass. */
	void SetCurrentTestPass(int32 PassIndex){ CurrentTestPass = PassIndex; }

	/**
	 * Export the automation report.
	 *
	 * @param FileExportTypeMask The file export mask - errors, warning etc.
	 * @param NumDeviceClusters The number of devices in a cluster.
	 */
	const bool ExportReport( uint32 FileExportTypeMask, const int32 NumDeviceClusters );

protected:

	/**
	 * Used in report generation to find leaf reports.
	 *
	 * @param InReport The report to add.
 	 * @param NumDeviceClusters - The number of devices in a cluster.
	 * @param ResultsLog - Message log to store the test results.
	 * @param FileExportTypeMask - The file export mask - errors, warning etc.
	 */
	void AddResultReport( TSharedPtr< IAutomationReport > InReport, const int32 NumDeviceClusters, TArray< FString >& ResultsLog, uint32 FileExportTypeMask );

	/**
	 * Used in report generation to find leaf reports.
	 *
	 * @param InReport The parent report.
	 * @param NumDeviceClusters The number of devices in a cluster.
	 * @param ResultsLog Message log to store the test results.
	 * @param FileExportTypeMask The file export mask - errors, warning etc.
	 */
	void FindLeafReport( TSharedPtr< IAutomationReport > InReport, const int32 NumDeviceClusters, TArray< FString >& ResultsLog, uint32 FileExportTypeMask );


private:

	/** Root node of the hierarchy, just there to ensure little code duplication. */
	TSharedPtr <IAutomationReport> ReportRoot;

	/** Which test pass we are currently on. */
	int32 CurrentTestPass;
};
