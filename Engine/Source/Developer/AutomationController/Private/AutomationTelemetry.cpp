// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTelemetry.h"

#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/DateTime.h"
#include "Misc/CommandLine.h"
#include "AutomationControllerSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationTelemetry, Log, All)

bool FAutomationTelemetry::bIsInitialized;
FString FAutomationTelemetry::TelemetryDirectory;
bool FAutomationTelemetry::bResetTelemetryStorageOnNewSession;

bool CreateDirectoryRecursively(const FString& FilePath)
{
	FString DirectoryPath = FPaths::GetPath(FilePath);
	if (FPaths::IsDrive(DirectoryPath))
	{
		return true;
	}

	IFileManager& GFileManager = IFileManager::Get();
	if (!GFileManager.DirectoryExists(*DirectoryPath))
	{
		if (!CreateDirectoryRecursively(DirectoryPath))
		{
			return false;
		}

		GFileManager.MakeDirectory(*DirectoryPath);
		if (!GFileManager.DirectoryExists(*DirectoryPath))
		{
			// Could not make the specified directory
			return false;
		}
	}

	return true;
}

bool SaveStringToNewFile(const FString& FilePath, FString& SaveText)
{
	if (!CreateDirectoryRecursively(FilePath))
	{
		return false;
	}

	return FFileHelper::SaveStringToFile(*SaveText, *FilePath);
}

FString FAutomationTelemetry::ToColumnValue(uint8 Index, const FAutomationWorkerTelemetryData& Data, const FAutomationWorkerTelemetryItem& Item)
{
	switch (Index)
	{
	case Columns::Configuration:
		return Data.Configuration;

	case Columns::Platform:
		return Data.Platform;

	case Columns::DateTime:
		return FDateTime::Now().ToString();

	case Columns::TestName:
		return Data.TestName;

	case Columns::Context:
		return Item.Context;

	case Columns::DataPoint:
		return Item.DataPoint;

	case Columns::Measurement:
		return FString::SanitizeFloat(Item.Measurement);

	default:
		return TEXT("NULL");
	}
}

void FAutomationTelemetry::Initialize()
{

	const UAutomationControllerSettings* Settings = GetDefault<UAutomationControllerSettings>();
	bResetTelemetryStorageOnNewSession = Settings->bResetTelemetryStorageOnNewSession;

	FString TelemetryDirectoryFromSettings;
	FParse::Value(FCommandLine::Get(), TEXT("TelemetryDirectory="), TelemetryDirectoryFromSettings, false);
	if (TelemetryDirectoryFromSettings.IsEmpty())
	{
		TelemetryDirectoryFromSettings = Settings->TelemetryDirectory;
	}

	if (TelemetryDirectoryFromSettings.IsEmpty())
	{
		TelemetryDirectory = FPaths::AutomationDir() + TEXT("Telemetry");
	}
	else
	{
		if (FPaths::IsRelative(TelemetryDirectoryFromSettings))
		{
			TelemetryDirectory = FPaths::AutomationDir() + TelemetryDirectoryFromSettings;
		}
		else
		{
			TelemetryDirectory = TelemetryDirectoryFromSettings;
		}
	}

	bIsInitialized = true;
}

bool FAutomationTelemetry::IsInitialized()
{
	return bIsInitialized;
}

FString FAutomationTelemetry::GetStorageFilePath(const FString& StorageName)
{
	return FPaths::SetExtension(TelemetryDirectory / (StorageName.IsEmpty()? TEXT("general") : StorageName), "csv");
}

bool FAutomationTelemetry::InitiateStorage(const FString& StorageFilePath)
{
	if (bResetTelemetryStorageOnNewSession || !FPaths::FileExists(StorageFilePath))
	{
		FString Header;
		for (uint8 i = 0; i < Columns::Count; i++)
		{
			Header += ToColumnName(i);
			if (i < Columns::Count - 1)
			{
				Header += TEXT(",");
			}
		}
		Header += LINE_TERMINATOR;

		if (!SaveStringToNewFile(StorageFilePath, Header))
		{
			return false;
		}
	}

	return true;
}

void FAutomationTelemetry::HandleAddTelemetry(const FAutomationWorkerTelemetryData& Data)
{
	if (!IsInitialized())
	{
		Initialize();
	}

	UE_LOGF(LogAutomationTelemetry, Verbose, "Received request for %ls", *Data.TestName);

	FString StorageFilePath = GetStorageFilePath(Data.Storage);
	if (!InitiateStorage(StorageFilePath))
	{
		UE_LOGF(LogAutomationTelemetry, Error, "Could not create Telemetry file '%ls'", *StorageFilePath);
		return;
	}

	FString Row;
	for (const FAutomationWorkerTelemetryItem& Item : Data.Items)
	{
		for (uint8 i = 0; i < Columns::Count; i++)
		{
			Row += ToColumnValue(i, Data, Item);
			if (i < Columns::Count - 1)
			{
				Row += TEXT(",");
			}
		}
		Row += LINE_TERMINATOR;
	}

	if (!FFileHelper::SaveStringToFile(*Row, *StorageFilePath, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), FILEWRITE_Append))
	{
		UE_LOGF(LogAutomationTelemetry, Error, "Could not write in Telemetry file '%ls'", *StorageFilePath);
	}
}
