// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "AutomationWorkerMessages.h"


class FAutomationTelemetry
	: FNoncopyable
{
public:

	/** Called to initialize the singleton. */
	static void Initialize();
	
	/** Helper to check if the Analytics provider is Initialized */
	static bool IsInitialized();

public:

	/** Handle adding telemetry data to output */
	static void HandleAddTelemetry(const FAutomationWorkerTelemetryData& Data);

private:
	
	static bool bIsInitialized;
	static FString TelemetryDirectory;
	static bool bResetTelemetryStorageOnNewSession;

	enum Columns : uint8
	{
		Configuration,
		Platform,
		DateTime,
		TestName,
		Context,
		DataPoint,
		Measurement,

		Count // number of columns
	};

	static FString ToColumnName(uint8 Index)
	{
		switch (Index)
		{
		case Columns::Configuration:
			return TEXT("Configuration");

		case Columns::Platform:
			return TEXT("Platform");

		case Columns::DateTime:
			return TEXT("DateTime");

		case Columns::TestName:
			return TEXT("TestName");

		case Columns::Context:
			return TEXT("Context");

		case Columns::DataPoint:
			return TEXT("DataPoint");

		case Columns::Measurement:
			return TEXT("Measurement");

		default:
			return TEXT("Unknown");
		}
	}

	static FString ToColumnValue(uint8 Index, const FAutomationWorkerTelemetryData& Data, const FAutomationWorkerTelemetryItem& Item);

	/** Initiate telemetry storage csv file */
	static bool InitiateStorage(const FString& StorageName);

	/** Get telemetry storage file path using automation settings */
	static FString GetStorageFilePath(const FString& StorageName);

};
