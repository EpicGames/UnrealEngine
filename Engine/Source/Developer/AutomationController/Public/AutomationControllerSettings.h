// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IAutomationReport.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "AutomationControllerSettings.generated.h"

#define UE_API AUTOMATIONCONTROLLER_API

/*
* Describes a base filter for a test group.
*/
USTRUCT()
struct FAutomatedTestFilterBase
{
	GENERATED_BODY()

public:

	FAutomatedTestFilterBase(FString InContains, bool InMatchFromStart = false, bool InMatchFromEnd = false)
		: Contains(InContains), MatchFromStart(InMatchFromStart), MatchFromEnd(InMatchFromEnd)
	{
	}

	FAutomatedTestFilterBase() : FAutomatedTestFilterBase(TEXT("")) {}

	virtual ~FAutomatedTestFilterBase() = default;

	/** String that the test must contain */
	UPROPERTY(Config)
		FString Contains;

	/** If true start matching from the start of the string, else anywhere */
	UPROPERTY(Config)
		bool MatchFromStart;

	/** If true start matching from the end of the string, else anywhere */
	UPROPERTY(Config)
		bool MatchFromEnd;

	virtual bool PassesFilter(const TSharedPtr< IAutomationReport >& InReport) const
	{
		bool bMeetsMatch = true;

		if (MatchFromStart || MatchFromEnd)
		{
			if (MatchFromStart)
			{
				bMeetsMatch = InReport->GetFullTestPath().StartsWith(Contains);
			}

			if (MatchFromEnd && bMeetsMatch)
			{
				bMeetsMatch = InReport->GetFullTestPath().EndsWith(Contains);
			}
		}
		else
		{
			bMeetsMatch = InReport->GetFullTestPath().Contains(Contains);
		}

		return bMeetsMatch;
	}
};

/*
 * Describes a tag-based filter for tests
 */
USTRUCT()
struct FAutomatedTestTagFilter
{
	GENERATED_BODY()

public:

	/**
	 * @param InContains - String of concatenated tags and boolean operators
	 * 
	 * @see FTextFilterExpressionEvaluator
	 */
	FAutomatedTestTagFilter(const FString& InContains)
	{
		FString FilterString = InContains.TrimStartAndEnd();
		if (FilterString.IsEmpty())
		{
			TagFilter = nullptr;
		}
		else
		{
			TagFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);
			TagFilter->SetFilterText(FText::FromString(FilterString));
		}
	}

	virtual ~FAutomatedTestTagFilter() = default;

	FAutomatedTestTagFilter() : FAutomatedTestTagFilter(TEXT("")) {}

	virtual bool PassesFilter(const TSharedPtr< IAutomationReport >& InReport) const
	{
		if (TagFilter)
		{
			return TagFilter->TestTextFilter(FBasicStringFilterExpressionContext(InReport->GetTags()));
		}
		else // disabled filter, reject nothing
		{
			return true;
		}
	}

private:
	TSharedPtr<FTextFilterExpressionEvaluator> TagFilter;
};


/*
* Describes a filter for a test group with exclude and tag options.
*/
USTRUCT()
struct FAutomatedTestFilter : public FAutomatedTestFilterBase
{
	GENERATED_BODY()

public:

	FAutomatedTestFilter(FString InContains, bool InMatchFromStart = false, bool InMatchFromEnd = false)
		: FAutomatedTestFilterBase(InContains, InMatchFromStart, InMatchFromEnd)
	{
	}

	FAutomatedTestFilter() : FAutomatedTestFilter(TEXT("")) {}

	/** List of filters to exclude */
	UPROPERTY(Config)
		TArray<FAutomatedTestFilterBase> Exclude;

	/** List of tag filters specific to this group */
	UPROPERTY(Config)
		TArray<FAutomatedTestTagFilter> Tags;

	virtual bool PassesFilter(const TSharedPtr< IAutomationReport >& InReport) const override
	{
		bool bMeetsMatch = Super::PassesFilter(InReport);

		if (bMeetsMatch && !Exclude.IsEmpty())
		{
			// Apply exclusion rules
			for (const FAutomatedTestFilterBase& Filter : Exclude)
			{
				if (Filter.PassesFilter(InReport))
				{
					return false;
				}
			}
		}

		// Intersect with tag filters
		if (bMeetsMatch && !Tags.IsEmpty())
		{
			for (const FAutomatedTestTagFilter& Filter : Tags)
			{
				if (Filter.PassesFilter(InReport))
				{
					return true;
				}
			}
			bMeetsMatch = false; // failed to match any tag filter
		}

		return bMeetsMatch;
	}
};

/*
 *	Describes a group of tests. Each group has a name and a set of filters that determine group membership
 */
USTRUCT()
struct FAutomatedTestGroup
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(Config)
		FString Name;

	UPROPERTY(Config)
		TArray<FAutomatedTestFilter> Filters;
};



/**
 * Implements the Editor's user settings.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UAutomationControllerSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	
	/** List of user-defined test groups */
	UPROPERTY(Config)
	TArray<FAutomatedTestGroup> Groups;

	/** Whether to suppress log from test results (default=false) */
	UPROPERTY(Config)
	bool bSuppressLogErrors;

	/** Whether to suppress log warnings from test results (default=false) */
	UPROPERTY(Config)
	bool bSuppressLogWarnings;

	/** Whether to treat log warnings as log errors (default=true) */
	UPROPERTY(Config)
	bool bElevateLogWarningsToErrors;

	/** Log categories where warnings/errors will not affect the result of tests. A finer-grained way of preventing rogue systems from leading to test warnings/errors */
	UPROPERTY(Config)
	TArray<FString> SuppressedLogCategories;

	/** Whether to keep the PIE Open in the editor at the end of a test pass (default=false) */
	UPROPERTY(Config)
	bool bKeepPIEOpen;

	/** Whether to automatically expand Automation Tests tree subgroups that have single non-leaf item as a child (default=true) */
	UPROPERTY(Config)
	bool bAutoExpandSingleItemSubgroups;

	/** Whether to Sort tests by failure type in json/html report */
	UPROPERTY(Config)
	bool bSortTestsByFailure;

	/** Whether to prune log events from test report on success */
	UPROPERTY(Config)
	bool bPruneLogsOnSuccess;

private:
	/** Whether to treat log warnings as test errors (default=true) */
	UPROPERTY(Config, Meta = (DeprecatedProperty, DeprecationMessage = "Use bElevateLogWarningsToErrors instead."))
	bool bTreatLogWarningsAsTestErrors;
	
public:
	/** How long to wait between test updates (default=1sec)*/
	UPROPERTY(Config)
	float CheckTestIntervalSeconds;
	
	/** The maximum response wait time for detecting a lost game instance (default=300sec)*/
	UPROPERTY(Config)
	float GameInstanceLostTimerSeconds;

	/** Path to where telemetry files are saved (default=<project>/Saved/Automation/Telemetry/)*/
	UPROPERTY(Config)
	FString TelemetryDirectory;

	/** Whether to reset data stored in telemetry file (default=false) */
	UPROPERTY(Config)
	bool bResetTelemetryStorageOnNewSession;

	UE_API virtual void PostInitProperties() override;

	/** List of pre-selected tests to filter by when requesting tests */
	TArray<FString> InitialSelectedTestsNames;

	/** Prepare to open test tag editor when a list of InitialSelectedTestsNames is provided */
	bool bOpenTagEditorWhenInitialSelectedTestsNames;
};

#undef UE_API
