// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAutomationReport.h"
#include "Misc/IFilter.h"

class FAutomationFilter
	: public IFilter<const TSharedPtr<class IAutomationReport>&>
{
public:

	/** Default constructor. */
	FAutomationFilter()
		: OnlySmokeTests( false )
		, ShowErrors( false )
		, ShowWarnings( false )
		, OnlyExcludedTests ( false )
	{ }

public:

	/**
	 * Set if we should only show warnings.
	 *
	 * @param InShowWarnings If we should show warnings or not.
	 * @see SetOnlyShowSmokeTests, SetShowErrors, ShouldShowWarnings
	 */
	void SetShowWarnings( const bool InShowWarnings )
	{
		ShowWarnings = InShowWarnings;
	}

	/**
	 * Should we show warnings.
	 *
	 * @return True if we should be showing warnings.
	 * @see SetShowErrors, SetShowWarnings
	 */
	const bool ShouldShowWarnings( ) const
	{
		return ShowWarnings;
	}

	/**
	 * Set if we should only show errors.
	 *
	 * @param InShowErrors If we should show errors.
	 * @see SetOnlyShowSmokeTests, SetShowWarnings, ShouldShowErrors
	 */
	void SetShowErrors( const bool InShowErrors )
	{
		ShowErrors = InShowErrors;
	}

	/**
	 * Should we show errors.
	 *
	 * @return True if we should be showing errors.
	 * @see SetShowErrors, ShouldShowWarnings
	 */
	const bool ShouldShowErrors( ) const
	{
		return ShowErrors;
	}

	/**
	 * Set if we should only show smoke tests.
	 *
	 * @param InOnlySmokeTests If we should show smoke tests.
	 */
	void SetOnlyShowSmokeTests( const bool InOnlySmokeTests )
	{
		OnlySmokeTests = InOnlySmokeTests;
	}

	/**
	 * Should we only show errors.
	 *
	 * @return True if we should be showing errors.
	 * @see SetOnlyShowSmokeTests, ShouldShowErrors, ShouldShowWarnings
	 */
	const bool OnlyShowSmokeTests( ) const
	{
		return OnlySmokeTests;
	}

	/**
	 * Should we only show excluded tests.
	 *
	 * @return True if we should be showing only excluded tests.
	 * @see SetOnlyShowSmokeTests, ShouldShowErrors, ShouldShowWarnings
	 */
	const bool ShouldShowOnlyExcludedTests() const
	{
		return OnlyExcludedTests;
	}

	/**
	 * Set if we should only show excluded tests.
	 *
	 * @param InOnlyExcludedTests If we should show excluded tests.
	 */
	void SetShowOnlyExcludedTests(const bool InOnlyExcludedTests)
	{
		OnlyExcludedTests = InOnlyExcludedTests;
	}

public:

	// IFilter interface

	DECLARE_DERIVED_EVENT(FAutomationFilter, IFilter< const TSharedPtr< class IAutomationReport >& >::FChangedEvent, FChangedEvent);
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	virtual bool PassesFilter( const TSharedPtr< IAutomationReport >& InReport ) const override
	{
		bool FilterPassed = true;

		if (OnlySmokeTests)
		{
			//if we only want smoke tests and this isn't one
			if (!InReport->IsSmokeTest())
			{
				return false;
			}
			//Leaf Nodes dictate this matching, not root nodes
			if (InReport->GetTotalNumChildren())
			{
				return false;
			}
		}

		if (OnlyExcludedTests)
		{
			if (!InReport->IsToBeSkipped())
			{
				return false;
			}
		}

		if (ShowWarnings && ShowErrors)
		{
			FilterPassed = InReport->HasWarnings() || InReport->HasErrors();
		}
		else if (!ShowWarnings && ShowErrors)
		{
			FilterPassed = InReport->HasErrors();
		}
		else if (ShowWarnings && !ShowErrors)
		{
			// Do not show report as a warning if it should be highlighted as an Error!
			FilterPassed = InReport->HasWarnings() && !InReport->HasErrors();
		}

		return FilterPassed;
	}

private:

	/**	The event that broadcasts whenever a change occurs to the filter. */
	FChangedEvent ChangedEvent;
	
	/** Only smoke tests will pass the test. */
	bool OnlySmokeTests;

	/** Only errors will pass the test. */
	bool ShowErrors;

	/** Only warnings will pass the test. */
	bool ShowWarnings;

	/** Only exclude tests. */
	bool OnlyExcludedTests;
};
