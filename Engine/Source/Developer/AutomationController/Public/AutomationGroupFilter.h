// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAutomationReport.h"
#include "Misc/IFilter.h"
#include "AutomationControllerSettings.h"

class FAutomationGroupFilter
	: public IFilter<const TSharedPtr<class IAutomationReport>&>
{
public:

	/** Default constructor. */
	FAutomationGroupFilter()
	{ }

	/** Default constructor with array param. */
	FAutomationGroupFilter(const TArray<FAutomatedTestFilter>& InFilters)
		: Filters(InFilters)
	{ }

	/** Default constructor, group with single elements. */
	FAutomationGroupFilter(const FAutomatedTestFilter& InFilter)
	{
		Filters.Add(InFilter);
	}

public:
	/**
	 * Set the list of strings the group filter checks for substrings in test display name.
	 *
	 * @param InFilters An array of strings to filter against test display names.
	 * @see FAutomatedTestFilterBase::MatchFromStart, FAutomatedTestFilterBase::MatchFromEnd
	 */
	void SetFilters(const TArray<FAutomatedTestFilter>& InFilters)
	{
		Filters = InFilters;
		ChangedEvent.Broadcast();
	}

	/**
	 * Set the list of search syntax strings evaluated against test tags.
	 *
	 * @param InFilters An array of strings to filter against test tags.
	 * @see FTextFilterExpressionEvaluator
	 */
	void SetTagFilter(const TArray < FAutomatedTestTagFilter>& InFilters)
	{
		TagFilters = InFilters;
		ChangedEvent.Broadcast();
	}
public:

	// IFilter interface

	DECLARE_DERIVED_EVENT(FAutomationGroupFilter, IFilter< const TSharedPtr< class IAutomationReport >& >::FChangedEvent, FChangedEvent);
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	virtual bool PassesFilter(const TSharedPtr< IAutomationReport >& InReport) const override
	{
		// empty filters pass against all values
		bool NameFilterPassing = Filters.IsEmpty();
		bool TagFilterPassing = TagFilters.IsEmpty();

		for (const FAutomatedTestFilter& NameFilter: Filters)
		{
			if (NameFilter.PassesFilter(InReport))
			{
				NameFilterPassing = true;
				break;
			}
		}

		for (const FAutomatedTestTagFilter& TagFilter : TagFilters)
		{
			if (TagFilter.PassesFilter(InReport))
			{
				TagFilterPassing = true;
				break;
			}
		}

		return NameFilterPassing && TagFilterPassing;
	}

private:

	/**	The event that broadcasts whenever a change occurs to the filter. */
	FChangedEvent ChangedEvent;

	/** The array of FAutomatedTestFilter to filter against test names. At least one from the list must be matched. */
	TArray<FAutomatedTestFilter> Filters;

	/** The array of FAutomatedTestTagFilter to filter against test tags. At least one from the list must be matched. */
	TArray<FAutomatedTestTagFilter> TagFilters;

};
