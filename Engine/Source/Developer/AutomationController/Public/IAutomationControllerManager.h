// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAutomationReport.h"

class IAutomationControllerManager;

/** Type definition for shared pointers to instances of IAutomationControllerManager. */
typedef TSharedPtr<class IAutomationControllerManager> IAutomationControllerManagerPtr;

/** Type definition for shared references to instances of IAutomationControllerManager. */
typedef TSharedRef<class IAutomationControllerManager> IAutomationControllerManagerRef;


// Enum to indicate the state of the workers in the automation controller
namespace EAutomationControllerModuleState 
{
	enum Type
	{
		// Has workers available
		Ready,
		// Is running tests
		Running,
		// No workers available
		Disabled
	};
};


/**
* Enum used to set which reports to export
*/
namespace EFileExportType
{
	enum Type
	{
		FET_Status		= 0x00000001, // Export test status
		FET_Warnings	= 0x00000002, // Export warnings
		FET_Errors		= 0x00000004, // Export Errors
		FET_Logs		= 0x00000008, // Export logs
		FET_All			= 0x00000010  // Export everything
	};

	// Helper functions to set and check export flags
	
	/**
	* Check if a flag is set
	*
	* @param InMask The mask to check
	* @param InFlag The flag to check
	* @return True if the flag is set
	*/
	FORCEINLINE static bool IsSet( const uint32& InMask, const EFileExportType::Type InFlag )
	{
		if ( InMask & ( 1<<(uint32)InFlag ) )
		{
			return true;
		}
		return false;
	}

	/**
	* Remove a flag
	*
	* @param InMask The mask to unset the flag on
	* @param InFlag The flag to remove
	*/
	FORCEINLINE static void RemoveFlag( uint32& InMask, const EFileExportType::Type InFlag )
	{
		InMask &= ~( 1<<(uint32)InFlag );
	}

	/**
	* Set a flag
	*
	* @param InMask The mask to set the flag on
	* @param InFlag The flag to set
	*/
	FORCEINLINE static void SetFlag( uint32& InMask, const EFileExportType::Type InFlag )
	{
		InMask |= ( 1<<(uint32)InFlag );
	}

};


namespace EAutomationDeviceGroupTypes
{
	enum Type
	{
		MachineName,	// Group by machine name
		Platform,		// Group by platform
		OSVersion,		// Group by operating system version
		Model,			// Group by machine model
		GPU,			// Group by GPU
		CPUModel,		// Group by CPU Model
		RamInGB,		// Group by RAM in gigabytes
		RenderMode,		// Group by RenderMode (D3D11_SM5, OpenGL_SM4, etc)
		Max
	};

	static FText ToName( const Type DeviceGroupType )
	{
		switch( DeviceGroupType )
		{
		case MachineName:	return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_MachineName", "Machine Name");
		case Platform:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_Platform", "Platform");
		case OSVersion:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_OSVersion", "OS Version");
		case Model:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_Model", "Model");
		case GPU:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_GPU", "GPU");
		case CPUModel:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_CPUModel", "CPU Model");
		case RamInGB:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_RAM", "RAM in GB");
		case RenderMode:	return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_RenderMode", "Render Mode");

		default:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_InvalidOrMax", "InvalidOrMax");
		}
	}

	static FText ToDescription(const Type DeviceGroupType)
	{
		switch( DeviceGroupType )
		{
		case MachineName:	return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_MachineName", "Group devices based off their machine name");
		case Platform:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_Platform", "Group devices based off their platform");
		case OSVersion:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_OSVersion", "Group devices based off their OS version");
		case Model:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_Model", "Group devices based off their device model");
		case GPU:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_GPU", "Group devices based off their GPU");
		case CPUModel:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_CPUModel", "Group devices based off their CPU model");
		case RamInGB:		return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_RAM", "Group devices based off memory (in GB)");
		case RenderMode:	return NSLOCTEXT("AutomationTest", "AutomationDeviceGroupTip_RenderMode", "Group devices based off the current render mode");

		default:			return NSLOCTEXT("AutomationTest", "AutomationDeviceGroup_InvalidOrMax", "InvalidOrMax");
		}
	}
}


/** Console command delegate type (takes no arguments.)  This is a void callback function. */
DECLARE_MULTICAST_DELEGATE(FOnAutomationControllerReset);

/** Console command delegate type (takes no arguments.)  This is a void callback function. */
DECLARE_MULTICAST_DELEGATE(FOnAutomationControllerTestsRefreshed);

/** Delegate for setting the automation controller status in the UI */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAutomationControllerManagerTestsAvailable, EAutomationControllerModuleState::Type);

/** Delegate to call when the automation controller is shutdown. */
DECLARE_MULTICAST_DELEGATE(FOnAutomationControllerManagerShutdown);

/** Delegate to call when the tests are complete. */
DECLARE_MULTICAST_DELEGATE(FOnAutomationControllerTestsComplete);

/** Delegate for receiving a test report upon test pass completion. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAutomationControllerReportGenerated, const FString&);


/**
 * Interface for AutomationController .
 */
class IAutomationControllerManager
{
public:

	/** 
	 * Broadcast a request for workers to ping if they are available.
	 * 
	 * @param InSessionID the session ID to run the test on.
	 */
	virtual void RequestAvailableWorkers( const FGuid& InSessionId ) = 0;

	/** 
	 * Return true when the manager is ready and capable of executing tests. 
	 * if RunTests is called earlier it may block until internally these conditions are met
	 */
	virtual bool IsReadyForTests() = 0;

	/** Send request for tests that are available to run. */
	virtual void RequestTests() = 0;

	/** 
	 * Resets all enabled tests to be able to run during Tick by local/remote machines.
	 *
	 * @param bIsLocalSeesion Indicate if this is a local session. We do not time out local session automation tests.
	 */
	virtual void RunTests( const bool bIsLocalSession = false ) = 0;

	/** Stops all running tests. */
	virtual void StopTests() = 0;

	/** Initializes the automation controller. */
	virtual void Init() = 0;

	/**
	 * Request an asset to be loaded in the editor.
	 *
	 * @param InAssetName The asset name.
	 */
	virtual void RequestLoadAsset( const FString& InAssetName ) = 0;

	/** Tick function that will execute enabled tests for different device clusters. */
	virtual void Tick() = 0;

	/**
	 * Sets the number of test passes.
	 *
	 * @param InNumPasses The number of test passes to perform.
	 */
	virtual void SetNumPasses(const int32 InNumPasses) = 0;

	/**
	 * Get the number of test passes.
	 *
	 * @return Number of passes.
	 */
	virtual int32 GetNumPasses() = 0;

	/**
	* Returns if analytics should be sent
	*/
	virtual bool IsSendAnalytics() const = 0;

	/**
	* Sets if analytics should be sent
	*/
	virtual void SetSendAnalytics(const bool bNewValue) = 0;

	/**
	* Returns if PIE should be kept open when test pass end
	*/
	virtual bool KeepPIEOpen() const = 0;

	/**
	* Sets if PIE should be kept open when test pass end
	*/
	virtual void SetKeepPIEOpen(const bool bNewValue) = 0;

	/**
	 * Filters the visible list of tests.
	 */
	virtual void SetFilter( TSharedPtr< AutomationFilterCollection > InFilter ) = 0;
	
	/**
	 * Gives the array of filtered test results to the UI.
	 */
	virtual TArray <TSharedPtr <IAutomationReport> >& GetFilteredReports() = 0;

	/**
	 * Gives the array of enabled test results to the UI.
	 */
	virtual TArray <TSharedPtr <IAutomationReport> > GetEnabledReports() = 0;

	/**
	 * Get num devices types.
	 */
	virtual int32 GetNumDeviceClusters() const = 0;

	/**
	 * Get num devices in specified cluster.
	 */
	virtual int32 GetNumDevicesInCluster(const int32 ClusterIndex) const = 0;

	/**
	 * Get the group name of the specified cluster.
	 */
	virtual FString GetClusterGroupName(const int32 ClusterIndex) const =0;

	/**
	 * Get name of a particular device cluster.
	 */
	virtual FString GetDeviceTypeName(const int32 ClusterIndex) const = 0;

	/**
	 * Get a game instance ID.
	 *
	 * @param ClusterIndex The cluster Index.
	 * @param DeviceIndex The Device Index.
	 * @return identifier of a game.
	 */
	virtual FGuid GetGameInstanceId(const int32 ClusterIndex, const int32 DeviceIndex) const = 0;

	/**
	 * Get a game instance name.
	 *
	 * @param ClusterIndex The cluster Index.
 	 * @param DeviceIndex The Device Index.
	 */
	virtual FString GetGameInstanceName(const int32 ClusterIndex, const int32 DeviceIndex) const = 0;

	/**
	 * Get a device name.
	 *
	 * @param ClusterIndex The cluster Index.
	 * @param DeviceIndex The Device Index.
	 */
	virtual FString GetDeviceName(const int32 ClusterIndex, const int32 DeviceIndex) const = 0;

	/**
	 * Sets whether all visible tests are enabled or not.
	 */
	virtual void SetVisibleTestsEnabled(const bool bEnabled) = 0;

	/**
	 * Returns number of tests that will be run.
	 */
	virtual int32 GetEnabledTestsNum() const = 0;

	/**
	 * Gets the names of all the enabled tests.
	 *
	 * @param OutEnabledTestNames The list to populate with enabled test names.
	 */
	virtual void GetEnabledTestNames(TArray<FString>& OutEnabledTestNames) const = 0;

	/**
	* Gets the names of all the filtered tests.
	*
	* @param OutFilteredTestNames The list to populate with filtered test names.
	*/
	virtual void GetFilteredTestNames(TArray<FString>& OutFilteredTestNames) const = 0;

	/**
	 * Sets any tests that match a name in the enabled tests array.
	 *
	 * @param EnabledTests An array of test names that will be enabled.
	 */
	virtual void SetEnabledTests(const TArray<FString>& EnabledTests) = 0;

	/**
	 * Gets the controller's current test state.
	 *
	 * @return The test state.
	 */
	virtual EAutomationControllerModuleState::Type GetTestState( ) const = 0;

	/**
	 * Sets whether the automation tests should include developer content directories.
	 */
	virtual void SetDeveloperDirectoryIncluded(const bool bInDeveloperDirectoryIncluded) = 0;

	/**
	 * Returns whether the automation tests should include developer content directories.
	 */
	virtual bool IsDeveloperDirectoryIncluded(void) const = 0;

	/**
	 * Sets what flags we're considering automation tests from
	 */
	virtual void SetRequestedTestFlags(const EAutomationTestFlags InRequestedTestFlags) = 0;

	/**
	 * Check if the automation tests have completed.
	 *
	 * @return true if the tests are available.
	 */
	virtual const bool CheckTestResultsAvailable() const = 0;

	/**
	 * Check if the automation tests results have errors.
	 *
	 * @return true if the tests have errors.
	 */
	virtual const bool ReportsHaveErrors() const = 0;

	/**
	 * Check if the automation tests results have warnings.
	 *
	 * @return true if the tests have warnings.
	 */
	virtual const bool ReportsHaveWarnings() const = 0;

	/**
	 * Check if the automation tests results have logs.
	 *
	 * @return true if the tests have logs.
	 */
	virtual const bool ReportsHaveLogs() const = 0;

	/**
	 * Remove results from the automation controller module.
	 */
	virtual void ClearAutomationReports() = 0;

	/**
	 * Generate an automation report.
	 *
	 * @param FileExporTypeMask The types of report to export. Warning errors etc.
	 */
	virtual const bool ExportReport( uint32 FileExportTypeMask ) = 0;

	/** 
	 * Check that the test we are looking to run is runnable.
	 *
	 * @param InReport The test we are checking is runnable.
	 * @return true if the test can be run, false if not.
	 */
	virtual bool IsTestRunnable( IAutomationReportPtr InReport ) const = 0;

	/** Removes all callbacks. */
	virtual void RemoveCallbacks() = 0;

	/** Shuts down the manager. */
	virtual void Shutdown() = 0;

	/** Starts up the manager. */
	virtual void Startup() = 0;

	/** Checks if a device group flag is set. */
	virtual bool IsDeviceGroupFlagSet( EAutomationDeviceGroupTypes::Type InDeviceGroup ) const = 0;

	/** Toggles a device group flag. */
	virtual void ToggleDeviceGroupFlag( EAutomationDeviceGroupTypes::Type InDeviceGroup ) = 0;

	/** Updates the clusters when the device grouping changes. */
	virtual void UpdateDeviceGroups( ) = 0;

	virtual FString GetReportOutputPath() const = 0;

	virtual void ResetAutomationTestTimeout(const TCHAR* Reason) = 0;
	
public:

	/**
	 * Gets the multicast delegate that is invoked when the controller manager shuts down.
	 *
	 * @return The delegate.
	 */
	virtual FOnAutomationControllerManagerShutdown& OnShutdown( ) = 0;

	/**
	 * Gets the multicast delegate that is invoked when the controller has tests available.
	 *
	 * @return The delegate.
	 */
	virtual FOnAutomationControllerManagerTestsAvailable& OnTestsAvailable( ) = 0;

	/**
	 * Gets the multicast delegate that is invoked when the controller's test status changes.
	 *
	 * @return The delegate.
	 */
	virtual FOnAutomationControllerTestsRefreshed& OnTestsRefreshed( ) = 0;

	/**
	 * Gets the test complete multicast delegate.
	 */
	virtual FOnAutomationControllerTestsComplete& OnTestsComplete( ) = 0;

	/**
	* Gets a delegate that is invoked when the controller's reset.
	*
	* @return The delegate.
	*/
	virtual FOnAutomationControllerReset& OnControllerReset() = 0;

	/**
	* Gets a delegate that is invoked when the controller generates a report.
	*
	* @return The delegate.
	*/
	virtual FOnAutomationControllerReportGenerated& OnTestReportGenerated()
	{
		return TestReportGeneratedDelegate;
	}

public:

	/** Virtual destructor.*/
	virtual ~IAutomationControllerManager() = default;

protected:
	/** Holds a delegate that is invoked when the tests have generated a report. */
	FOnAutomationControllerReportGenerated TestReportGeneratedDelegate;
};
