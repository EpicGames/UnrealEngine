// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IAutomationControllerManager.h"

struct FAutomationExecutionEntry;

/**
 * Emit test results in the format Gauntlet's AutomationLogParser expects.
 * Shares the file-scoped format #defines with ReportAutomationResult in
 * AutomationControllerManager.cpp. Both call sites use the same defines
 * but are not yet consolidated into a single code path.
 *
 * TODO: Refactor ReportAutomationResult to call these functions, making
 * them the single entry point for Gauntlet protocol emission.
 */
struct FAutomationResultEmitter
{
	AUTOMATIONCONTROLLER_API static void EmitTestStarted(const FString& Name, const FString& Path);
	AUTOMATIONCONTROLLER_API static void EmitTestCompleted(const FString& State, const FString& Name, const FString& Path);
	AUTOMATIONCONTROLLER_API static void EmitBeginEvents(const FString& Path);
	AUTOMATIONCONTROLLER_API static void EmitEndEvents(const FString& Path);
	AUTOMATIONCONTROLLER_API static void EmitEntry(const FAutomationExecutionEntry& Entry);
};

/**
 * Interface for AutomationController modules.
 */
class IAutomationControllerModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the automation controller.
	 *
	 * @return a reference to the automation controller.
	 */
	virtual IAutomationControllerManagerRef GetAutomationController( ) = 0;

	/** Init message bus usage. */
	virtual void Init() = 0;

	/** Tick function that will execute enabled tests for different device clusters. */
	virtual void Tick() = 0;

	static IAutomationControllerModule& Get()
	{
		return FModuleManager::GetModuleChecked<IAutomationControllerModule>("AutomationController");
	}
};
