// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "AutomationTestExcludelist.h"
#include "AutomationState.h"
#include "UObject/ObjectMacros.h"
#include "IAutomationReport.generated.h"

class IAutomationReport;
template< typename ItemType > class TFilterCollection;

/**
* The automation filter collection - used for updating the automation report list
*/
typedef TFilterCollection< const TSharedPtr< class IAutomationReport >& > AutomationFilterCollection;


typedef TSharedPtr<class IAutomationReport> IAutomationReportPtr;

typedef TSharedRef<class IAutomationReport> IAutomationReportRef;


UENUM()
enum class EAutomationArtifactType : uint8
{
	None,
	Image,
	Comparison
};

namespace ComparisonFileTypes
{
	// Use const string here instead of an enum, because FAutomationArtifact.Files property is a dictionary string/string
	// FAutomationArtifact should be able to be used as an adhoc key/file association.
	const FString Unapproved = TEXT("unapproved");
	const FString Approved = TEXT("approved");
	const FString Difference = TEXT("difference");
}

USTRUCT()
struct FAutomationArtifact
{
	GENERATED_BODY()

public:
	FAutomationArtifact()
		: Name()
		, Type(EAutomationArtifactType::None)
	{
	}

	FAutomationArtifact(FGuid InUniqueId, const FString& InName, EAutomationArtifactType InType, const TMap<FString, FString>& InLocalFiles)
		: Id(InUniqueId)
		, Name(InName)
		, Type(InType)
		, LocalFiles(InLocalFiles)
	{
	}

public:

	UPROPERTY()
	FGuid Id;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	EAutomationArtifactType Type;

	UPROPERTY()
	TMap<FString, FString> Files;

	// Local Files are the files generated during a testing run, once exported, the individual file paths
	// should be stored in the Files map.

	TMap<FString, FString> LocalFiles;
};

/**
 * A struct to maintain a collection of data which was reported as part of an automation test result.
 **/
struct FAutomationTestResults
{
	/**
	 * Default Constructor
	 **/
	FAutomationTestResults()
		: State( EAutomationState::NotRun ) //default to not having run yet
		, Duration( 0.0f )
		, WarningTotal(0)
		, ErrorTotal(0)
	{
	}

	void Reset()
	{
		State = EAutomationState::NotRun;
		Entries.Empty();
		Artifacts.Empty();
		WarningTotal = 0;
		ErrorTotal = 0;
	}

	void SetEvents(const TArray<FAutomationExecutionEntry>& InEvents, int32 InWarningTotal, int32 InErrorTotal)
	{
		Entries = InEvents;
		WarningTotal = InWarningTotal;
		ErrorTotal = InErrorTotal;
	}

	void AddEvent(const FAutomationExecutionEntry& Event)
	{
		switch ( Event.Event.Type )
		{
		case EAutomationEventType::Warning:
			WarningTotal++;
			break;
		case EAutomationEventType::Error:
			ErrorTotal++;
			break;
		}

		Entries.Add(Event);
	}

	const TArray<FAutomationExecutionEntry>& GetEntries() const { return Entries; }

	int32 GetLogTotal() const { return Entries.Num() - (WarningTotal + ErrorTotal); }
	int32 GetWarningTotal() const { return WarningTotal; }
	int32 GetErrorTotal() const { return ErrorTotal; }

public:
	/* The current state of this test */
	EAutomationState State;

	/* The time this test took to complete */
	float Duration;

	/* The instance which reported these results */
	FString GameInstance;

	/** Artifacts generated during the run of the test. */
	TArray<FAutomationArtifact> Artifacts;

private:
	/* All events reported as part of this test */
	TArray<FAutomationExecutionEntry> Entries;

	int32 WarningTotal;

	int32 ErrorTotal;
};


/** Intermediate structure for calculating how complete a automation test is */
struct FAutomationCompleteState
{
	FAutomationCompleteState()
		: NumEnabledTestsPassed(0)
		, NumEnabledTestsWarnings(0)
		, NumEnabledTestsFailed(0)
		, NumEnabledTestsCouldntBeRun(0)
		, NumEnabledInProcess(0)
		, TotalEnabled(0)
		, NumDisabledTestsPassed(0)
		, NumDisabledTestsWarnings(0)
		, NumDisabledTestsFailed(0)
		, NumDisabledTestsCouldntBeRun(0)
	{		
	}

	// Stats for enabled tests
	uint32 NumEnabledTestsPassed;
	uint32 NumEnabledTestsWarnings;
	uint32 NumEnabledTestsFailed;
	uint32 NumEnabledTestsCouldntBeRun;
	uint32 NumEnabledInProcess;
	uint32 TotalEnabled;

	//stats for disabled tests
	uint32 NumDisabledTestsPassed;
	uint32 NumDisabledTestsWarnings;
	uint32 NumDisabledTestsFailed;
	uint32 NumDisabledTestsCouldntBeRun;
};

/**
 * Interface for automation test results
 */
class IAutomationReport : public TSharedFromThis<IAutomationReport>
{
public:
	/** virtual destructor */
	virtual ~IAutomationReport() = default;

	/** Remove all child tests */
	virtual void Empty() = 0;

	/**
	 * Returns the complete command for an automation test including any relevant parameters.  This is the class name + the parameter.
	 */
	virtual FString GetCommand() const = 0;

/**
	 * Returns the name of this level in the test hierarchy for the purposes of grouping.
	 * (Editor.Maps.LoadAll.parameters) would have 3 internal tree nodes and a variable number of leaf nodes depending on how many maps existed
	 *
	 * @param bAddDecoration true is the name should have the number of child tests appended.
	 * @return the name of this level in the hierarchy for use in UI.
	 */
	virtual const FString& GetDisplayName() const = 0;

	/**
	 * Returns the full path for the test, e.g. System.Audio.PlaySoundTest.
	 */
	virtual const FString& GetFullTestPath() const = 0;

	/**
	 * Returns the name of this level in the test hierarchy for the purposes of UI.
	 * (Editor.Maps.LoadAll.parameters) would have 3 internal tree nodes and a variable number of leaf nodes depending on how many maps existed
	 *
	 * @return the name of this level in the hierarchy for use in UI.
	 */
	virtual FString GetDisplayNameWithDecoration() const = 0;

	/**
	 * Get the name of the asset associated with this test.
	 * 
	 * @return the asset name.
	 */
	virtual FString GetTestParameter() const = 0;

	/**
	 * Get the tags associated with this test.
	 *
	 * @return the tags concatenated into a single string.
	 */
	virtual FString GetTags() const = 0;

	/**
	 * Get the tags associated with this test, as an array.
	 *
	 * @return the tags as a string array.
	 */
	virtual TArray<FString> GetTagsArray() const = 0;

	/**
	 * Gets the asset path associated with a test, it may not have one.
	 * 
	 * @return the asset name.
	 */
	virtual FString GetAssetPath() const = 0;

	virtual FString GetOpenCommand() const = 0;

	/**
	 * Get the test type.
	 * 
	 * @return the test type.
	 */
	virtual EAutomationTestFlags GetTestFlags() const = 0;

	/** Gets the source file the test was defined on. */
	virtual FString GetSourceFile() const = 0;

	/** Gets the source file line number the test was defined on. */
	virtual int32 GetSourceFileLine() const = 0;

	/** Recursively gets the number of child nodes */
	virtual int32 GetTotalNumChildren() const = 0;

	/** Recursively gets the total number of filtered children */
	virtual int32 GetTotalNumFilteredChildren() const = 0;

	/** Gets the names of all the enabled tests */
	virtual void GetEnabledTestNames(TArray<FString>& OutEnabledTestNames, FString CurrentPath) const = 0;

	/** Gets the names of all the filtered tests */
	virtual void GetFilteredTestNames(TArray<FString>& OutFilteredTestNames, FString CurrentPath) const = 0;

	/** Sets which tests are enabled based off the enabled tests list */
	virtual void SetEnabledTests(const TArray<FString>& EnabledTests, FString CurrentPath) = 0;

	/** Recursively gets the number of enabled tests */
	virtual int32 GetEnabledTestsNum() const = 0;

	/** Return if this test should be executed */
	virtual bool IsEnabled() const = 0;

	/**
	 * Is this a parent type.
	 * 
	 * @return true if this is a parent type.
	 */
	virtual const bool IsParent() = 0;

	/**
	 * Is this a smoke test.
	 * 
	 * @return true if this is a smoke test.
	 */
	virtual const bool IsSmokeTest() = 0;

	/** Sets whether this test should be executed or not. */
	virtual void SetEnabled(bool bShouldBeEnabled) = 0;

	/** Sets whether this test is supported on a particular platform. */
	virtual void SetSupport(const int32 ClusterIndex) = 0;

	/**
	 * Set the test flags.
	 * 
	 * @param InTestFlags The EAutomationTestFlags of the test.
	 */
	virtual void SetTestFlags(const EAutomationTestFlags InTestFlags) = 0;

	/** Returns if a particular platform is supported */
	virtual bool IsSupported(const int32 ClusterIndex) const = 0;

	/**
	 * Filters the visible tests based on search text, execution status, regression test, etc.
	 *
	 * @param InFilterDesc The description of how to decide if this report should be visible.
	 * @param ParentPassedFilter If the parent passed the filter.
	 * @return whether this report or any of its children passed the filter.
	 */
	virtual bool SetFilter( TSharedPtr< AutomationFilterCollection > InFilter, const bool ParentPassedFilter = false ) = 0;

	/** Returns the array of child reports that should be visible to the UI based on filtering. */
	virtual TArray<TSharedPtr<IAutomationReport> >& GetFilteredChildren() = 0;

	/** Returns the array of child reports. */
	virtual TArray<TSharedPtr<IAutomationReport> >& GetChildReports() = 0;

	/** Updates the report when the number of clusters changes. */
	virtual void ClustersUpdated(const int32 NumClusters) = 0;

	/** 
	 * Recursively resets the report to "needs to be run", clears cached warnings and errors.
	 *
	 * @param NumTestPasses The number of test passes so we know how many results to create.
	 */
	virtual void ResetForExecution(const int32 NumTestPasses) = 0;

	/**
	 * Sets the results of the test for use by the UI.
	 *
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param PassIndex Which test pass these results are for.
	 * @param InResults The new set of results.
	 */
	virtual void SetResults(const int32 ClusterIndex, const int32 PassIndex, const FAutomationTestResults& InResults) = 0;

	virtual void AddArtifact(const int32 ClusterIndex, const int32 PassIndex, const FAutomationArtifact& Artifact) = 0;

	/**
	 * Returns completion statistics for this branch of the testing hierarchy.
	 *
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param PassIndex Which test pass to get the status of.
	 * @param OutCompletionState Collection structure for execution statistics.
	 */
	virtual void GetCompletionStatus(const int32 ClusterIndex, const int32 PassIndex, FAutomationCompleteState& OutCompletionState) = 0;

	/**
	 * Returns the state of the test (not run, in process, success, failure).
	 *
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param PassIndex Which test pass to get the state of.
	 * @return the current state of the test.
	 */
	virtual EAutomationState GetState(const int32 ClusterIndex, const int32 PassIndex) const = 0;

	/**
	 * Set the state of the test (not run, in process, success, failure).
	 *
	 * @param State EAutomationState state to set.
	 */
	virtual void SetState(const EAutomationState State) = 0;

	/**
	 * Gets a copy of errors and warnings that were found
	 *
	 * @param ClusterIndex Index of the platform we are requesting test results for.
	 * @param PassIndex Index of the test pass to get the results for.
	 * @return The collection of results for the given cluster index
	 */
	virtual const FAutomationTestResults& GetResults( const int32 ClusterIndex, const int32 PassIndex ) = 0;

	/**
	 * Gets the number of available test results for a given cluster.
	 *
	 * @param ClusterIndex Index of the platform.
	 * @return The number of results available for this cluster
	 */
	virtual const int32 GetNumResults( const int32 ClusterIndex ) = 0;

	/**
	 * Finds the current pass by looking at the current state
	 * @param ClusterIndex - Index of the platform.
	 * @return The current pass index
	 */
	virtual const int32 GetCurrentPassIndex( const int32 ClusterIndex ) = 0;

	/**
	 * Gets the name of the instance that ran the test.
	 *
	 * @param ClusterIndex - Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @return the name of the device.
	 */
	virtual FString GetGameInstanceName(const int32 ClusterIndex) = 0;
	
	/**
	 * Add a child test to the hierarchy, creating internal tree nodes as needed.
	 * If NewTestName is Editor.Maps.Loadall.TestName, this will create nodes for Editor, Maps, Loadall, and then a leaf node for the test name with the associated command line
	 *
	 * @param TestInfo Structure containing all the test info.
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param NumPasses The number of passes we are going to perform.  Used to make sure we have enough results..
	 * @return The automation report.
	 */
	virtual TSharedPtr<IAutomationReport> EnsureReportExists(FAutomationTestInfo& TestInfo, const int32 ClusterIndex, const int32 NumPasses) = 0;

	/**
	 * Returns the next test in the hierarchy to run.
	 *
	 * @param bOutAllTestsComplete Whether or not all enabled tests have completed execution for this platform (cluster index).
	 * @param ClusterIndex Index of the platform reporting the results of this test.  See AutomationDeviceClusterManager.
	 * @param PassIndex Which test pass we are currently on.
	 * @param NumDevicesInCluster The number of devices which are in this cluster.
	 * @return The next report
	 */
	virtual TSharedPtr<IAutomationReport> GetNextReportToExecute(bool& bOutAllTestsComplete, const int32 ClusterIndex, const int32 PassIndex, const int32 NumDevicesInCluster) = 0;

	/**
	 * Returns all the enabled test reports.
	 *
	 * @param an array to be filled with all reports
	 */
	virtual void GetEnabledTestReports(TArray<TSharedPtr<IAutomationReport>>& OutReports) = 0;

	/**
	 * Returns if there have been any errors in the test.
	 *
	 * @return True If there were errors.
	 */
	virtual const bool HasErrors() = 0;

	/**
	 * Returns if there have been any warnings in the test.
	 *
	 * @return True If there were errors.
	 */
	virtual const bool HasWarnings() = 0;

	/**
	 * Gets the min and max time this test took to execute.
	 *
	 * @param OutMinTime Minimum execution time for all device types.
	 * @param OutMaxTime Maximum execution time for all device types.
	 * @return whether any test has completed successfully.
	 */
	virtual const bool GetDurationRange(float& OutMinTime, float& OutMaxTime) = 0;

	/**
	 * Get the number of devices which have been given this test to run.
	 *
	 * @return The number of devices who have been given this test to run.
	 */
	virtual const int32 GetNumDevicesRunningTest() const = 0;

	/**
	 * Get the number of participants this test requires.
	 *
	 * @Return The number of devices needed for this test to execute.
	 */
	virtual const int32 GetNumParticipantsRequired() const = 0;
	
	/**
	 * Set the number of participants this test requires if less than what is already set.
	 *
	 * @param NewCount The number of further devices .
	 */
	virtual void SetNumParticipantsRequired( const int32 NewCount ) = 0;

	/**
	 * Increment the number of network responses.
	 *
	 * @return true if responses from all participants were received, false otherwise.
	 */
	virtual bool IncrementNetworkCommandResponses() = 0;

	/*** Resets the number of network responses back to zero. */
	virtual void ResetNetworkCommandResponses() = 0;

	/**
	 * Should we expand this node in the UI - A child has passed the filter.
	 *
	 * @return true If we should expand the node
	 */
	virtual const bool ExpandInUI() const = 0;

	/** Stop the test which is creating this report. */
	virtual void StopRunningTest() = 0;

	/**
	* Is the test need to be skipped.
	*
	* @param OutReasaon the related reason of the exclusion information
	* @param OutWarn the related warning of the exclusion
	* @return true if the test is inside exclude list.
	*/
	virtual bool IsToBeSkipped(FName* OutReason = nullptr, bool* OutWarn = nullptr) const = 0;

	/**
	* Is the test need to be skipped on specific conditions.
	*
	* @return true if the test is inside exclude list.
	*/
	virtual bool IsToBeSkippedOnConditions() const = 0;

	/**
	* Is the test is to be skipped through propagation.
	*
	* @return true if the test is inside exclude list via a propagation.
	*/
	virtual bool IsToBeSkippedByPropagation() const = 0;
	
	/**
	* Add or remove test from exclude list.
	*
	* @param if true, add this item to the exclude list, remove otherwise.
	*/
	virtual void SetSkipFlag(bool bEnableSkip, const FAutomationTestExcludelistEntry* Template = nullptr, bool bFromPropagation = false) = 0;

	/**
	* Produce exclude options .
	*
	* @return exclude options based on report exclude info.
	*/
	virtual TSharedPtr<FAutomationTestExcludeOptions> GetExcludeOptions() = 0;


	// Event that allows log to refresh once a test has finished
	DECLARE_DELEGATE_OneParam(FOnSetResultsEvent, TSharedPtr<IAutomationReport>);
	FOnSetResultsEvent OnSetResults;
};
